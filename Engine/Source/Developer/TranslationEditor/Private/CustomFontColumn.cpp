// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR

#include "CustomFontColumn.h"

#include "HAL/PlatformMath.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyPath.h"
#include "SlotBase.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyHandle;
class IPropertyTableUtilities;
class SWidget;

#define LOCTEXT_NAMESPACE "PropertyTable.CustomFontColumn"

bool FCustomFontColumn::Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const
{
	bool IsSupported = false;
	
	if( Column->GetDataSource()->IsValid() )
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if( PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0 )
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			FProperty* Property = PropertyInfo.Property.Get();
			if (SupportedProperties.Contains(Property))
			{
				IsSupported = true;
			}
		}
	}

	return IsSupported;
}

TSharedPtr< SWidget > FCustomFontColumn::CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	return 
	SNew( SHorizontalBox )
	+SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign( HAlign_Left )
	.VAlign( VAlign_Center )
	.FillWidth(2)
	[
		SNew( STextBlock )
		.Font( FAppStyle::GetFontStyle( Style ) )
		.Text( Column->GetDisplayName() )
	]
	+SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign( HAlign_Center )
	.VAlign( VAlign_Center )
	.FillWidth(2)
	[
		SNew( SButton )
		.HAlign( HAlign_Center )
		.VAlign( VAlign_Center )
		.OnClicked(OnChangeFontButtonClicked)
		[
				SNew( STextBlock )
				.Text((FText::Format(LOCTEXT("ChooseFontButton", "Choose {0} Font: "), Column->GetDisplayName())))
		]
	]
	+SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign( HAlign_Center )
	.VAlign( VAlign_Center )
	.FillWidth(1)
	.Padding( 0.0f, 4.0f, 0.0f, 4.0f )
	[
		SNew( SSpinBox<int32> )
		.Delta(1)
		.MinValue(1)
		.MaxValue(100)
		.Value(Font.Size)
		.OnValueCommitted( OnFontSizeValueCommitted )
	]
	;
}

TSharedPtr< IPropertyTableCellPresenter > FCustomFontColumn::CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if( PropertyHandle.IsValid() )
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
		return PropertyEditorModule.CreateTextPropertyCellPresenter(Cell->GetNode().ToSharedRef(), Utilities, &Font);
	}

	return NULL;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
