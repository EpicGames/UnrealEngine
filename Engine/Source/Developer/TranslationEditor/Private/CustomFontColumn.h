// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR

#include "Containers/Array.h"
#include "UserInterface/PropertyTable/PropertyTableConstants.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTableCustomColumn.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FProperty;
class IPropertyTableCell;
class IPropertyTableCellPresenter;
class IPropertyTableColumn;
class IPropertyTableUtilities;
class SWidget;

/**
 * A property table custom column used to display text cells in a chosen font
 */
class FCustomFontColumn : public IPropertyTableCustomColumn
{
public:
	FCustomFontColumn(FSlateFontInfo InFont = FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle ), FOnClicked InOnChangeFontButtonClicked = NULL, FOnInt32ValueCommitted InOnFontSizeValueCommitted = NULL)
		:	Font(InFont)
		,	OnChangeFontButtonClicked(InOnChangeFontButtonClicked)
		,	OnFontSizeValueCommitted(InOnFontSizeValueCommitted)
	{}

	/** Begin IPropertyTableCustomColumn interface */
	virtual bool Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const override;
	virtual TSharedPtr< SWidget > CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	/** End IPropertyTableCustomColumn interface */

private:

	/** Font to use for this column's cells */
	FSlateFontInfo Font;

	/** List of properties that this Custom Font Column should be used to display */
	TArray<FProperty*> SupportedProperties;

	/** Function to call when Change Font button clicked */
	FOnClicked OnChangeFontButtonClicked;

	/** Function to call when Font Size SpinBox's Value is committed */
	FOnInt32ValueCommitted OnFontSizeValueCommitted;

public:
	void SetFont(FSlateFontInfo InFont)
	{
		Font = InFont;
	}

	void AddSupportedProperty(FProperty* Property)
	{
		SupportedProperties.Add(Property);
	}

	void SetOnChangeFontButtonClicked(FOnClicked InOnClicked)
	{
		OnChangeFontButtonClicked = InOnClicked;
	}

	void SetOnFontSizeValueCommitted(FOnInt32ValueCommitted InOnFontSizeValueCommitted)
	{
		OnFontSizeValueCommitted = InOnFontSizeValueCommitted;
	}
};

#endif // WITH_EDITOR
