// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITranslationEditor.h"
#include "TranslationPickerWidget.h"
#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "TranslationEditorModule.h"
#include "LocalizationConfigurationScript.h"

/** To keep track of what translation editors are open editing which archive files */
TMap<FString, ITranslationEditor*> ITranslationEditor::OpenTranslationEditors = TMap<FString, ITranslationEditor*>();

void ITranslationEditor::OpenTranslationEditor(const FString& InManifestFile, const FString& InNativeArchiveFile, const FString& InArchiveFileToEdit)
{
	// Only open a new translation editor if there isn't another one open for this archive file already
	if (!OpenTranslationEditors.Contains(InArchiveFileToEdit))
	{
		FTranslationEditorModule& TranslationEditorModule = FModuleManager::LoadModuleChecked<FTranslationEditorModule>("TranslationEditor");
		bool bLoadedSuccessfully = false;
		ITranslationEditor* NewTranslationEditor = (ITranslationEditor*)&(TranslationEditorModule.CreateTranslationEditor(InManifestFile, InNativeArchiveFile, InArchiveFileToEdit, bLoadedSuccessfully).Get());

		NewTranslationEditor->RegisterTranslationEditor();

		if (!bLoadedSuccessfully)
		{
			NewTranslationEditor->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
		}
	}
	else
	{
		// If already editing this archive file, flash the tab that contains the editor that has that file open
		ITranslationEditor* OpenEditor = *OpenTranslationEditors.Find(InArchiveFileToEdit);
		OpenEditor->FocusWindow();
	}
}

void ITranslationEditor::OpenTranslationEditor(ULocalizationTarget* const LocalizationTarget, const FString& CultureToEdit)
{
	check(LocalizationTarget);
	const FString ManifestFile = LocalizationConfigurationScript::GetManifestPath(LocalizationTarget);
	FString NativeCultureName;
	if (LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
	{
		NativeCultureName = LocalizationTarget->Settings.SupportedCulturesStatistics[LocalizationTarget->Settings.NativeCultureIndex].CultureName;
	}
	const FString NativeArchiveFile = NativeCultureName.IsEmpty() ? FString() : LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, NativeCultureName);
	const FString ArchiveFile = LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, CultureToEdit);

	// Only open a new translation editor if there isn't another one open for this archive file already
	if (!OpenTranslationEditors.Contains(ArchiveFile))
	{
		FTranslationEditorModule& TranslationEditorModule = FModuleManager::LoadModuleChecked<FTranslationEditorModule>("TranslationEditor");
		bool bLoadedSuccessfully = false;
		ITranslationEditor* NewTranslationEditor = (ITranslationEditor*)&(TranslationEditorModule.CreateTranslationEditor(LocalizationTarget, CultureToEdit, bLoadedSuccessfully).Get());

		NewTranslationEditor->RegisterTranslationEditor();

		if (!bLoadedSuccessfully)
		{
			NewTranslationEditor->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
		}
	}
	else
	{
		// If already editing this archive file, flash the tab that contains the editor that has that file open
		ITranslationEditor* OpenEditor = *OpenTranslationEditors.Find(ArchiveFile);
		OpenEditor->FocusWindow();
	}
}

void ITranslationEditor::RegisterTranslationEditor()
{
	ITranslationEditor::OpenTranslationEditors.Add(ArchiveFilePath, this);
}

void ITranslationEditor::UnregisterTranslationEditor()
{
	OpenTranslationEditors.Remove(ArchiveFilePath);
}

void ITranslationEditor::OnClose()
{
	ITranslationEditor::UnregisterTranslationEditor();
}

#endif // WITH_EDITOR

void ITranslationEditor::OpenTranslationPicker()
{
	if (!TranslationPickerManager::IsPickerWindowOpen())
	{
		TranslationPickerManager::OpenPickerWindow();
	}
}
