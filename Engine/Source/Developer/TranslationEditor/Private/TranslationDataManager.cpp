// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslationDataManager.h"
#include "PortableObjectPipeline.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationArchive.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Styling/AppStyle.h"
#include "SourceControlOperations.h"
#include "ISourceControlState.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "TranslationUnit.h"
#include "Logging/MessageLog.h"
#include "TextLocalizationResourceGenerator.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Internationalization/Culture.h"
#include "PortableObjectFormatDOM.h"
#include "ILocalizationServiceModule.h"
#include "LocalizationModule.h"
#include "LocalizationTargetTypes.h"
#include "LocalizationConfigurationScript.h"
#include "Serialization/JsonInternationalizationArchiveSerializer.h"
#include "Serialization/JsonInternationalizationManifestSerializer.h"
#include "GeneralProjectSettings.h"


DEFINE_LOG_CATEGORY_STATIC(LogTranslationEditor, Log, All);

#define LOCTEXT_NAMESPACE "TranslationDataManager"

struct FLocTextIdentity
{
public:
	FLocTextIdentity(FLocKey InNamespace, FLocKey InKey)
		: Namespace(MoveTemp(InNamespace))
		, Key(MoveTemp(InKey))
	{
	}

	FORCEINLINE const FLocKey& GetNamespace() const
	{
		return Namespace;
	}

	FORCEINLINE const FLocKey& GetKey() const
	{
		return Key;
	}

	FORCEINLINE bool operator==(const FLocTextIdentity& Other) const
	{
		return Namespace == Other.Namespace && Key == Other.Key;
	}

	FORCEINLINE bool operator!=(const FLocTextIdentity& Other) const
	{
		return Namespace != Other.Namespace || Key != Other.Key;
	}

	friend inline uint32 GetTypeHash(const FLocTextIdentity& Id)
	{
		return HashCombine(GetTypeHash(Id.Namespace), GetTypeHash(Id.Key));
	}

private:
	FLocKey Namespace;
	FLocKey Key;
};

FTranslationDataManager::FTranslationDataManager( const FString& InManifestFilePath, const FString& InNativeArchiveFilePath, const FString& InArchiveFilePath )
	: OpenedManifestFilePath(InManifestFilePath)
	, NativeArchiveFilePath(InNativeArchiveFilePath)
	, OpenedArchiveFilePath(InArchiveFilePath)
	, bLoadedSuccessfully(true)
{
	Initialize();
}

FTranslationDataManager::FTranslationDataManager(ULocalizationTarget* const LocalizationTarget, const FString& CultureToEdit)
	: bLoadedSuccessfully(true)
{
	check(LocalizationTarget);
 
	const FString ManifestFile = LocalizationConfigurationScript::GetManifestPath(LocalizationTarget);
	FString NativeCultureName;
	if (LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
	{
		NativeCultureName = LocalizationTarget->Settings.SupportedCulturesStatistics[LocalizationTarget->Settings.NativeCultureIndex].CultureName;
	}
	const FString NativeArchiveFile = NativeCultureName.IsEmpty() ? FString() : LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, NativeCultureName);
	const FString ArchiveFileToEdit = LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, CultureToEdit);

	OpenedManifestFilePath = ManifestFile;
	NativeArchiveFilePath = NativeArchiveFile;
	OpenedArchiveFilePath = ArchiveFileToEdit;

	Initialize();
}

void FTranslationDataManager::Initialize()
{
	GWarn->BeginSlowTask(LOCTEXT("LoadingTranslationData", "Loading Translation Data..."), true);
	TArray<UTranslationUnit*> TranslationUnits;

	ManifestAtHeadRevisionPtr = ReadManifest( OpenedManifestFilePath );
	if (ManifestAtHeadRevisionPtr.IsValid())
	{
		TSharedRef< FInternationalizationManifest > ManifestAtHeadRevision = ManifestAtHeadRevisionPtr.ToSharedRef();
		int32 ManifestEntriesCount = ManifestAtHeadRevision->GetNumEntriesBySourceText();

		if (ManifestEntriesCount < 1)
		{
			bLoadedSuccessfully = false;
			FFormatNamedArguments Arguments;
			Arguments.Add( TEXT("ManifestFilePath"), FText::FromString(OpenedManifestFilePath) );
			Arguments.Add( TEXT("ManifestEntriesCount"), FText::AsNumber(ManifestEntriesCount) );
			FMessageLog TranslationEditorMessageLog("TranslationEditor");
			TranslationEditorMessageLog.Error(FText::Format(LOCTEXT("CurrentManifestEmpty", "Most current translation manifest ({ManifestFilePath}) has {ManifestEntriesCount} entries."), Arguments));
			TranslationEditorMessageLog.Notify(LOCTEXT("TranslationLoadError", "Error Loading Translations!"));
			TranslationEditorMessageLog.Open(EMessageSeverity::Error);
		}

		ArchivePtr = ReadArchive(OpenedArchiveFilePath);
		NativeArchivePtr = NativeArchiveFilePath != OpenedArchiveFilePath ? ReadArchive(NativeArchiveFilePath) : ArchivePtr;

		if (ArchivePtr.IsValid())
		{
			int32 NumManifestEntriesParsed = 0;

			GWarn->BeginSlowTask(LOCTEXT("LoadingCurrentManifest", "Loading Entries from Current Translation Manifest..."), true);

			// Get all manifest entries by source text...
			for (auto ManifestItr = ManifestAtHeadRevision->GetEntriesBySourceTextIterator(); ManifestItr; ++ManifestItr, ++NumManifestEntriesParsed)
			{
				GWarn->StatusUpdate(NumManifestEntriesParsed, ManifestEntriesCount, FText::Format(LOCTEXT("LoadingCurrentManifestEntries", "Loading Entry {0} of {1} from Current Translation Manifest..."), FText::AsNumber(NumManifestEntriesParsed), FText::AsNumber(ManifestEntriesCount)));
				const TSharedRef<FManifestEntry> ManifestEntry = ManifestItr.Value();
				TMap< FLocTextIdentity, UTranslationUnit* > IdentityToTranslationUnitMap;

				for(auto ContextIter( ManifestEntry->Contexts.CreateConstIterator() ); ContextIter; ++ContextIter)
				{
					FTranslationContextInfo ContextInfo;
					const FManifestContext& AContext = *ContextIter;

					ContextInfo.Context = AContext.SourceLocation;
					ContextInfo.Key = AContext.Key.GetString();
#if WITH_EDITORONLY_DATA
					ContextInfo.DevNotes = AContext.DevNotes;
#endif

					// Make sure we have a unique translation unit for each unique identity.
					UTranslationUnit*& TranslationUnit = IdentityToTranslationUnitMap.FindOrAdd(FLocTextIdentity(ManifestEntry->Namespace, AContext.Key));
					if (!TranslationUnit)
					{
						TranslationUnit = NewObject<UTranslationUnit>();
						check(TranslationUnit != nullptr);
						// We want Undo/Redo support
						TranslationUnit->SetFlags(RF_Transactional);
						TranslationUnit->HasBeenReviewed = false;
						TranslationUnit->Source = ManifestEntry->Source.Text;
						TranslationUnit->Namespace = ManifestEntry->Namespace.GetString();
						TranslationUnit->Key = AContext.Key.GetString();
						TranslationUnit->KeyMetaDataObject = AContext.KeyMetadataObj;
					}
					
					if (NativeArchivePtr.IsValid() && NativeArchivePtr != ArchivePtr)
					{
						const TSharedPtr<FArchiveEntry> NativeArchiveEntry = NativeArchivePtr->FindEntryByKey(ManifestEntry->Namespace, AContext.Key, AContext.KeyMetadataObj);
						// If the native archive contains a translation for the source string that isn't identical to the source string, use the translation as the source string.
						if (NativeArchiveEntry.IsValid() && !NativeArchiveEntry->Translation.IsExactMatch(NativeArchiveEntry->Source))
						{
							TranslationUnit->Source = NativeArchiveEntry->Translation.Text;
						}
					}

					TranslationUnit->Contexts.Add(ContextInfo);
				}


				TArray<UTranslationUnit*> TranslationUnitsToAdd;
				IdentityToTranslationUnitMap.GenerateValueArray(TranslationUnitsToAdd);
				TranslationUnits.Append(TranslationUnitsToAdd);
			}
			GWarn->EndSlowTask();

			LoadFromArchive(TranslationUnits);
		}
		else  // ArchivePtr.IsValid() is false
		{
			bLoadedSuccessfully = false;
			FFormatNamedArguments Arguments;
			Arguments.Add( TEXT("ArchiveFilePath"), FText::FromString(OpenedArchiveFilePath) );
			FMessageLog TranslationEditorMessageLog("TranslationEditor");
			TranslationEditorMessageLog.Error(FText::Format(LOCTEXT("FailedToLoadCurrentArchive", "Failed to load most current translation archive ({ArchiveFilePath}), unable to load translations."), Arguments));
			TranslationEditorMessageLog.Notify(LOCTEXT("TranslationLoadError", "Error Loading Translations!"));
			TranslationEditorMessageLog.Open(EMessageSeverity::Error);
		}
	}
	else  // ManifestAtHeadRevisionPtr.IsValid() is false
	{
		bLoadedSuccessfully = false;
		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("ManifestFilePath"), FText::FromString(OpenedManifestFilePath) );
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Error(FText::Format(LOCTEXT("FailedToLoadCurrentManifest", "Failed to load most current translation manifest ({ManifestFilePath}), unable to load translations."), Arguments));
		TranslationEditorMessageLog.Notify(LOCTEXT("TranslationLoadError", "Error Loading Translations!"));
		TranslationEditorMessageLog.Open(EMessageSeverity::Error);
	}

	GWarn->EndSlowTask();
}

FTranslationDataManager::~FTranslationDataManager()
{
	RemoveTranslationUnitArrayfromRoot(AllTranslations); // Re-enable garbage collection for all current UTranslationDataObjects
}

TSharedPtr< FInternationalizationManifest > FTranslationDataManager::ReadManifest( const FString& ManifestFilePathToRead )
{
	TSharedPtr<FInternationalizationManifest> InternationalizationManifest = MakeShareable(new FInternationalizationManifest);

	if (!FJsonInternationalizationManifestSerializer::DeserializeManifestFromFile(ManifestFilePathToRead, InternationalizationManifest.ToSharedRef()))
	{
		UE_LOGF(LogTranslationEditor, Error, "Could not read manifest file %ls.", *ManifestFilePathToRead);
		InternationalizationManifest.Reset();
	}

	return InternationalizationManifest;
}

TSharedPtr< FInternationalizationArchive > FTranslationDataManager::ReadArchive(const FString& ArchiveFilePath)
{
	TSharedPtr<FInternationalizationArchive> InternationalizationArchive = MakeShareable(new FInternationalizationArchive);

	if (!FJsonInternationalizationArchiveSerializer::DeserializeArchiveFromFile(ArchiveFilePath, InternationalizationArchive.ToSharedRef(), ManifestAtHeadRevisionPtr, nullptr))
	{
		UE_LOGF(LogTranslationEditor, Error, "Could not read archive file %ls.", *ArchiveFilePath);
		InternationalizationArchive.Reset();
	}

	return InternationalizationArchive;
}

bool FTranslationDataManager::WriteTranslationData(bool bForceWrite /*= false*/)
{
	bool bSuccess = false;

	// If the archive hasn't been loaded correctly, don't try and write anything
	if (ArchivePtr.IsValid())
	{
		TSharedRef< FInternationalizationArchive > Archive = ArchivePtr.ToSharedRef();

		bool bNeedsWrite = false;

		auto WriteTranslationUnits = [&Archive , &bNeedsWrite](const TArray<UTranslationUnit*> InTranslationUnits)
		{
			for (UTranslationUnit* TranslationUnit : InTranslationUnits)
			{
				if (TranslationUnit)
				{
					TSharedPtr<FArchiveEntry> OldArchiveEntry = Archive->FindEntryByKey(TranslationUnit->Namespace, TranslationUnit->Key, TranslationUnit->KeyMetaDataObject);
					if (TranslationUnit->HasBeenReviewed && (!OldArchiveEntry || !OldArchiveEntry->Source.Text.Equals(TranslationUnit->Source, ESearchCase::CaseSensitive) || !OldArchiveEntry->Translation.Text.Equals(TranslationUnit->Translation, ESearchCase::CaseSensitive)))
					{
						Archive->SetTranslation(TranslationUnit->Namespace, TranslationUnit->Key, FLocItem(TranslationUnit->Source), FLocItem(TranslationUnit->Translation), TranslationUnit->KeyMetaDataObject);
						bNeedsWrite = true;
					}
				}
			}
		};

		WriteTranslationUnits(Untranslated);
		WriteTranslationUnits(Review);
		WriteTranslationUnits(Complete);

		bSuccess = true;

		if (bForceWrite || bNeedsWrite)
		{
			FString FinalArchiveJson;
			FJsonInternationalizationArchiveSerializer::SerializeArchiveToString(Archive, FinalArchiveJson);

			bSuccess = WriteJSONToTextFile(FinalArchiveJson, OpenedArchiveFilePath);
		}
	}

	return bSuccess;
}

bool FTranslationDataManager::WriteJSONToTextFile(const FString& Output, const FString& Filename)
{
	bool CheckoutAndSaveWasSuccessful = true;
	bool bPreviouslyCheckedOut = false;

	// If the user specified a reference file - write the entries read from code to a ref file
	if ( !Filename.IsEmpty() )
	{
		// If source control is enabled, try to check out the file. Otherwise just try to write it
		if (ISourceControlModule::Get().IsEnabled())
		{
			// Already checked out?
			if (CheckedOutFiles.Contains(Filename))
			{
				bPreviouslyCheckedOut = true;
			}
			else if (!USourceControlHelpers::CheckOutOrAddFile(Filename))
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Filename"), FText::FromString(Filename));
				// Use Source Control Message Log here because there might be other useful information in that log for the user.
				FMessageLog SourceControlMessageLog("SourceControl");
				SourceControlMessageLog.Error(FText::Format(LOCTEXT("CheckoutFailed", "Check out of file '{Filename}' failed."), Arguments));
				SourceControlMessageLog.Notify(LOCTEXT("TranslationArchiveCheckoutFailed", "Failed to Check Out Translation Archive!"));
				SourceControlMessageLog.Open(EMessageSeverity::Error);
				CheckoutAndSaveWasSuccessful = false;
			}
			else
			{
				CheckedOutFiles.Add(Filename);
			}
		}

		if( CheckoutAndSaveWasSuccessful )
		{
			//Print the JSON data out to the ref file.
			if (!FFileHelper::SaveStringToFile(Output, *Filename, FFileHelper::EEncodingOptions::ForceUnicode))
			{
				// If we already checked out the file, but cannot write it, perhaps the user checked it in via perforce, so try to check it out again
				if (bPreviouslyCheckedOut)
				{
					bPreviouslyCheckedOut = false;

					if( !USourceControlHelpers::CheckOutOrAddFile(Filename) )
					{
						FFormatNamedArguments Arguments;
						Arguments.Add( TEXT("Filename"), FText::FromString(Filename) );
						// Use Source Control Message Log here because there might be other useful information in that log for the user.
						FMessageLog SourceControlMessageLog("SourceControl");
						SourceControlMessageLog.Error(FText::Format(LOCTEXT("CheckoutFailed", "Check out of file '{Filename}' failed."), Arguments));
						SourceControlMessageLog.Notify(LOCTEXT("TranslationArchiveCheckoutFailed", "Failed to Check Out Translation Archive!"));
						SourceControlMessageLog.Open(EMessageSeverity::Error);
						CheckoutAndSaveWasSuccessful = false;

						CheckedOutFiles.Remove(Filename);
					}
				}

				FFormatNamedArguments Arguments;
				Arguments.Add( TEXT("Filename"), FText::FromString(Filename) );
				FMessageLog TranslationEditorMessageLog("TranslationEditor");
				TranslationEditorMessageLog.Error(FText::Format(LOCTEXT("WriteFileFailed", "Failed to write localization entries to file '{Filename}'."), Arguments));
				TranslationEditorMessageLog.Notify(LOCTEXT("FileWriteFailed", "Failed to Write Translations to File!"));
				TranslationEditorMessageLog.Open(EMessageSeverity::Error);
				CheckoutAndSaveWasSuccessful = false;
			}
		}
	}
	else
	{
		CheckoutAndSaveWasSuccessful = false;
	}

	// If this is the first time, let the user know the file was checked out
	if (!bPreviouslyCheckedOut && CheckoutAndSaveWasSuccessful)
	{
		struct Local
		{
			/**
			* Called by our notification's hyperlink to open the Source Control message log
			*/
			static void OpenSourceControlMessageLog(  )
			{
				FMessageLog("SourceControl").Open();
			}
		};

		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("Filename"), FText::FromString(Filename) );

		// Make a note in the Source Control log, including a note to check in the file later via source control application
		FMessageLog TranslationEditorMessageLog("SourceControl");
		TranslationEditorMessageLog.Info(FText::Format(LOCTEXT("TranslationArchiveCheckedOut", "Successfully checked out and saved translation archive '{Filename}'. Please check-in this file later via your revision control application."), Arguments));

		// Display notification that save was successful, along with a link to the Source Control log so the user can see the above message.
		FNotificationInfo Info( LOCTEXT("ArchiveCheckedOut", "Translation Archive Successfully Checked Out and Saved.") );
		Info.ExpireDuration = 5;
		Info.Hyperlink = FSimpleDelegate::CreateStatic(&Local::OpenSourceControlMessageLog);
		Info.HyperlinkText = LOCTEXT("ShowMessageLogHyperlink", "Show Message Log");
		Info.bFireAndForget = true;
		Info.bUseSuccessFailIcons = true;
		Info.Image = FAppStyle::GetBrush(TEXT("NotificationList.SuccessImage"));
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	return CheckoutAndSaveWasSuccessful;
}

void FTranslationDataManager::GetHistoryForTranslationUnits()
{
	GWarn->BeginSlowTask(LOCTEXT("LoadingSourceControlHistory", "Loading Translation History from Revision Control..."), true);

	TArray<UTranslationUnit*>& TranslationUnits = AllTranslations;
	const FString& InManifestFilePath = OpenedManifestFilePath;

	// Unload any previous history information, going to retrieve it all again.
	UnloadHistoryInformation();

	// Force history update
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory( true );
	ECommandResult::Type Result = SourceControlProvider.Execute(UpdateStatusOperation, InManifestFilePath);
	bool bGetHistoryFromSourceControlSucceeded = Result == ECommandResult::Succeeded;

	// Now we can get information about the file's history from the source control state, retrieve that
	TArray<FString> Files;
	TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> > States;
	Files.Add(InManifestFilePath);
	Result = SourceControlProvider.GetState( Files,  States, EStateCacheUsage::ForceUpdate );
	bGetHistoryFromSourceControlSucceeded = bGetHistoryFromSourceControlSucceeded && (Result == ECommandResult::Succeeded);
	FSourceControlStatePtr SourceControlState;
	if (States.Num() == 1)
	{
		SourceControlState = States[0];
	}

	// If all the source control operations went ok, continue
	if (bGetHistoryFromSourceControlSucceeded && SourceControlState.IsValid())
	{
		int32 HistorySize = SourceControlState->GetHistorySize();

		for (int HistoryItemIndex = HistorySize-1; HistoryItemIndex >=0; --HistoryItemIndex)
		{
			GWarn->StatusUpdate(HistorySize - HistoryItemIndex, HistorySize, FText::Format(LOCTEXT("LoadingOldManifestRevisionNumber", "Loading Translation History from Manifest Revision {0} of {1} from Revision Control..."), FText::AsNumber(HistorySize - HistoryItemIndex), FText::AsNumber(HistorySize)));

			TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryItemIndex);
			if(Revision.IsValid())
			{
				FString ManifestFullPath = FPaths::ConvertRelativePathToFull(InManifestFilePath);
				FString EngineFullPath = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir());

				bool IsEngineManifest = false;
				if (ManifestFullPath.StartsWith(EngineFullPath))
				{
					IsEngineManifest = true;
				}

				FString ProjectName;
				FString SavedDir;	// Store these cached translation history files in the saved directory
				if (IsEngineManifest)
				{
					ProjectName = TEXT("Engine");
					SavedDir = FPaths::EngineSavedDir();
				}
				else
				{
					ProjectName = FApp::GetProjectName();
					SavedDir = FPaths::ProjectSavedDir();
				}

				FString TempFileName = SavedDir / TEXT("CachedTranslationHistory") / FString::Printf(TEXT("Manifest-%s-%s-Rev-%d"), *ProjectName, *FPaths::GetBaseFilename(InManifestFilePath), Revision->GetRevisionNumber());
				
				if (!FPaths::FileExists(TempFileName))	// Don't bother syncing again if we already have this manifest version cached locally
				{
					Revision->Get(TempFileName);
				}

				TSharedPtr< FInternationalizationManifest > OldManifestPtr = ReadManifest( TempFileName );
				if (OldManifestPtr.IsValid())	// There may be corrupt manifests in the history, so ignore them.
				{
					TSharedRef< FInternationalizationManifest > OldManifest = OldManifestPtr.ToSharedRef();

					for (UTranslationUnit* TranslationUnit : TranslationUnits)
					{
						if(TranslationUnit != nullptr && TranslationUnit->Contexts.Num() > 0)
						{
							for (FTranslationContextInfo& ContextInfo : TranslationUnit->Contexts)
							{
								FString PreviousSourceText;

								// If we already have history, then compare against the newest history so far
								if (ContextInfo.Changes.Num() > 0)
								{
									PreviousSourceText = ContextInfo.Changes[0].Source;
								}

								TSharedPtr< FManifestEntry > OldManifestEntryPtr = OldManifest->FindEntryByKey(TranslationUnit->Namespace, ContextInfo.Key);
								if (!OldManifestEntryPtr.IsValid())
								{
									// If this version of the manifest didn't know anything about this string, move onto the next
									continue;
								}

								// Always add first instance of this string, and then add any versions that changed since
								if (ContextInfo.Changes.Num() == 0 || !OldManifestEntryPtr->Source.Text.Equals(PreviousSourceText))
								{
									TSharedPtr< FArchiveEntry > OldArchiveEntry = ArchivePtr->FindEntryByKey(OldManifestEntryPtr->Namespace, ContextInfo.Key, nullptr);
									if (OldArchiveEntry.IsValid())
									{
										FTranslationChange Change;
										Change.Source = OldManifestEntryPtr->Source.Text;
										Change.Translation = OldArchiveEntry->Translation.Text;
										Change.DateAndTime = Revision->GetDate();
										Change.Version = FString::FromInt(Revision->GetRevisionNumber());
										ContextInfo.Changes.Insert(Change, 0);
									}
								}
							}
						}
					}
				}
				else // OldManifestPtr.IsValid() is false
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("ManifestFilePath"), FText::FromString(InManifestFilePath));
					Arguments.Add( TEXT("ManifestRevisionNumber"), FText::AsNumber(Revision->GetRevisionNumber()) );
					FMessageLog TranslationEditorMessageLog("TranslationEditor");
					TranslationEditorMessageLog.Warning(FText::Format(LOCTEXT("PreviousManifestCorrupt", "Previous revision {ManifestRevisionNumber} of {ManifestFilePath} failed to load correctly. Ignoring."), Arguments));
				}
			}
		}
	}
	// If source control operations failed, display error message
	else // (bGetHistoryFromSourceControlSucceeded && SourceControlState.IsValid()) is false
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ManifestFilePath"), FText::FromString(InManifestFilePath));
		FMessageLog TranslationEditorMessageLog("SourceControl");
		TranslationEditorMessageLog.Warning(FText::Format(LOCTEXT("SourceControlStateQueryFailed", "Failed to query revision control state of file {ManifestFilePath}."), Arguments));
		TranslationEditorMessageLog.Notify(LOCTEXT("RetrieveTranslationHistoryFailed", "Unable to Retrieve Translation History from Revision Control!"));
	}


	// Go though all translation units
	for (int32 CurrentTranslationUnitIndex = 0; CurrentTranslationUnitIndex < TranslationUnits.Num(); ++CurrentTranslationUnitIndex)
	{
		UTranslationUnit* TranslationUnit = TranslationUnits[CurrentTranslationUnitIndex];
		if (TranslationUnit != nullptr)
		{
			if (TranslationUnit->Translation.IsEmpty())
			{
				bool bHasTranslationHistory = false;
				int32 MostRecentNonNullTranslationIndex = -1;
				int32 ContextForRecentTranslation = -1;

				// Check all contexts for history
				for (int32 ContextIndex = 0; ContextIndex < TranslationUnit->Contexts.Num(); ++ContextIndex)
				{
					for (int32 ChangeIndex = 0; ChangeIndex < TranslationUnit->Contexts[ContextIndex].Changes.Num(); ++ChangeIndex)
					{
						if (!(TranslationUnit->Contexts[ContextIndex].Changes[ChangeIndex].Translation.IsEmpty()))
						{
							bHasTranslationHistory = true;
							MostRecentNonNullTranslationIndex = ChangeIndex;
							ContextForRecentTranslation = ContextIndex;
							break;
						}
					}

					if (bHasTranslationHistory)
					{
						break;
					}
				}

				// If we have history, but current translation is empty, this goes in the Needs Review tab
				if (bHasTranslationHistory)
				{
					// Offer the most recent translation (for the first context in the list) as a suggestion or starting point (not saved unless user checks "Has Been Reviewed")
					TranslationUnit->Translation = TranslationUnit->Contexts[ContextForRecentTranslation].Changes[MostRecentNonNullTranslationIndex].Translation;
					TranslationUnit->HasBeenReviewed = false;

					// Move from Untranslated to review
					if (Untranslated.Contains(TranslationUnit))
					{
						Untranslated.Remove(TranslationUnit);
					}
					if (!Review.Contains(TranslationUnit))
					{
						Review.Add(TranslationUnit);
					}
				}
			}
		}
	}


	GWarn->EndSlowTask();
}

void FTranslationDataManager::HandlePropertyChanged(FName PropertyName)
{
	// When a property changes, write the data so we don't lose changes if user forgets to save or editor crashes
	WriteTranslationData();
}

void FTranslationDataManager::PreviewAllTranslationsInEditor(ULocalizationTarget* LocalizationTarget)
{
	FString ManifestFullPath = FPaths::ConvertRelativePathToFull(OpenedManifestFilePath);
	FString EngineFullPath = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir());

	bool IsEngineManifest = false;
	if (ManifestFullPath.StartsWith(EngineFullPath))
	{
		IsEngineManifest = true;
	}

	if (LocalizationTarget != nullptr)
	{
		const FString ConfigFilePath = LocalizationConfigurationScript::GetRegenerateResourcesConfigPath(LocalizationTarget);
		LocalizationConfigurationScript::GenerateRegenerateResourcesConfigFile(LocalizationTarget).Write(ConfigFilePath);
		FTextLocalizationResourceGenerator::GenerateLocResAndUpdateLiveEntriesFromConfig(ConfigFilePath, EGenerateLocResFlags::None);
	}
	else
	{
		FText ErrorNotify = LOCTEXT("PreviewAllTranslationsInEditorFail", "Failed to preview translations in Editor!");
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Error(ErrorNotify);
		TranslationEditorMessageLog.Notify(ErrorNotify);
	}
}

void FTranslationDataManager::PopulateSearchResultsUsingFilter(const FString& SearchFilter)
{
	SearchResults.Empty();

	for (UTranslationUnit* TranslationUnit : AllTranslations)
	{
		if (TranslationUnit != nullptr)
		{
			bool bAdded = false;
			if (TranslationUnit->Source.Contains(SearchFilter) ||
				TranslationUnit->Translation.Contains(SearchFilter) || 
				TranslationUnit->Namespace.Contains(SearchFilter))
			{
				SearchResults.Add(TranslationUnit);
				bAdded = true;
			}
			
			for (FTranslationContextInfo CurrentContext : TranslationUnit->Contexts)
			{
				if (!bAdded && 
					(CurrentContext.Context.Contains(SearchFilter) ||
					CurrentContext.Key.Contains(SearchFilter)
#if WITH_EDITORONLY_DATA
					|| CurrentContext.DevNotes.Contains(SearchFilter)
#endif
					))
				{
					SearchResults.Add(TranslationUnit);
					break;
				}
			}
		}
	}
}

void FTranslationDataManager::LoadFromArchive(TArray<UTranslationUnit*>& InTranslationUnits, bool bTrackChanges /*= false*/, bool bReloadFromFile /*=false*/)
{
	GWarn->BeginSlowTask(LOCTEXT("LoadingArchiveEntries", "Loading Entries from Translation Archive..."), true);

	if (bReloadFromFile)
	{
		ArchivePtr = ReadArchive(OpenedArchiveFilePath);
		NativeArchivePtr = NativeArchiveFilePath != OpenedArchiveFilePath ? ReadArchive(NativeArchiveFilePath) : ArchivePtr;
	}

	if (ArchivePtr.IsValid())
	{
		const TSharedRef< FInternationalizationArchive > Archive = ArchivePtr.ToSharedRef();

		// Make a local copy of this array before we empty the arrays below (we might have been passed AllTranslations array)
		TArray<UTranslationUnit*> TranslationUnits;
		TranslationUnits.Append(InTranslationUnits);

		AllTranslations.Empty();
		Untranslated.Empty();
		Review.Empty();
		Complete.Empty();
		ChangedOnImport.Empty();

		for (int32 CurrentTranslationUnitIndex = 0; CurrentTranslationUnitIndex < TranslationUnits.Num(); ++CurrentTranslationUnitIndex)
		{
			UTranslationUnit* TranslationUnit = TranslationUnits[CurrentTranslationUnitIndex];
			if (TranslationUnit != nullptr)
			{
				if (!TranslationUnit->IsRooted())
				{
					TranslationUnit->AddToRoot(); // Disable garbage collection for UTranslationUnit objects
				}
				AllTranslations.Add(TranslationUnit);

				GWarn->StatusUpdate(CurrentTranslationUnitIndex, TranslationUnits.Num(), FText::Format(LOCTEXT("LoadingCurrentArchiveEntries", "Loading Entry {0} of {1} from Translation Archive..."), FText::AsNumber(CurrentTranslationUnitIndex), FText::AsNumber(TranslationUnits.Num())));

				TSharedPtr<FArchiveEntry> ArchiveEntry = Archive->FindEntryByKey(TranslationUnit->Namespace, TranslationUnit->Key, TranslationUnit->KeyMetaDataObject);
				if (ArchiveEntry.IsValid())
				{
					const FString PreviousTranslation = TranslationUnit->Translation;
					TranslationUnit->Translation.Reset();

					if (ArchiveEntry->Translation.Text.IsEmpty())
					{
						// No current translation - try and find an historic one
						bool bHasTranslationHistory = false;
						int32 MostRecentNonNullTranslationIndex = -1;
						int32 ContextForRecentTranslation = -1;

						for (int32 ContextIndex = 0; ContextIndex < TranslationUnit->Contexts.Num(); ++ContextIndex)
						{
							for (int32 ChangeIndex = 0; ChangeIndex < TranslationUnit->Contexts[ContextIndex].Changes.Num(); ++ChangeIndex)
							{
								if (!(TranslationUnit->Contexts[ContextIndex].Changes[ChangeIndex].Translation.IsEmpty()))
								{
									bHasTranslationHistory = true;
									MostRecentNonNullTranslationIndex = ChangeIndex;
									ContextForRecentTranslation = ContextIndex;
									break;
								}
							}

							if (bHasTranslationHistory)
							{
								break;
							}
						}

						// If we have history, but current translation is empty, this goes in the Needs Review tab
						if (bHasTranslationHistory)
						{
							// Offer the most recent translation (for the first context in the list) as a suggestion or starting point (not saved unless user checks "Has Been Reviewed")
							TranslationUnit->Translation = TranslationUnit->Contexts[ContextForRecentTranslation].Changes[MostRecentNonNullTranslationIndex].Translation;
							Review.Add(TranslationUnit);
						}
						else
						{
							Untranslated.Add(TranslationUnit);
						}
					}
					else if (!ArchiveEntry->Source.Text.Equals(TranslationUnit->Source, ESearchCase::CaseSensitive))
					{
						// If we have a stale translation, this goes in the Needs Review tab
						TranslationUnit->Translation = ArchiveEntry->Translation.Text;
						Review.Add(TranslationUnit);
					}
					else
					{
						TranslationUnit->Translation = ArchiveEntry->Translation.Text;
						TranslationUnit->HasBeenReviewed = true;
						Complete.Add(TranslationUnit);
					}

					// Add to changed array if we're tracking changes (i.e. when we import from .po files)
					if (bTrackChanges)
					{
						if (PreviousTranslation != TranslationUnit->Translation)
						{
							FString PreviousTranslationTrimmed = PreviousTranslation;
							PreviousTranslationTrimmed.TrimStartAndEndInline();
							FString CurrentTranslationTrimmed = TranslationUnit->Translation;
							CurrentTranslationTrimmed.TrimStartAndEndInline();
							// Ignore changes to only whitespace at beginning and/or end of string on import
							if (PreviousTranslationTrimmed.Equals(CurrentTranslationTrimmed))
							{
								TranslationUnit->Translation = PreviousTranslation;
							}
							else
							{
								ChangedOnImport.Add(TranslationUnit);
								TranslationUnit->TranslationBeforeImport = PreviousTranslation;
							}
						}
					}
				}
			}
		}

	}
	else  // ArchivePtr.IsValid() is false
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ArchiveFilePath"), FText::FromString(OpenedArchiveFilePath));
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Error(FText::Format(LOCTEXT("FailedToLoadCurrentArchive", "Failed to load most current translation archive ({ArchiveFilePath}), unable to load translations."), Arguments));
		TranslationEditorMessageLog.Notify(LOCTEXT("TranslationLoadError", "Error Loading Translations!"));
		TranslationEditorMessageLog.Open(EMessageSeverity::Error);
	}
	
	GWarn->EndSlowTask();
}

void FTranslationDataManager::RemoveTranslationUnitArrayfromRoot(TArray<UTranslationUnit*>& TranslationUnits)
{
	for (UTranslationUnit* TranslationUnit : TranslationUnits)
	{
		TranslationUnit->RemoveFromRoot();
	}
}

void FTranslationDataManager::UnloadHistoryInformation()
{
	TArray<UTranslationUnit*>& TranslationUnits = AllTranslations;

	for (int32 CurrentTranslationUnitIndex = 0; CurrentTranslationUnitIndex < TranslationUnits.Num(); ++CurrentTranslationUnitIndex)
	{
		UTranslationUnit* TranslationUnit = TranslationUnits[CurrentTranslationUnitIndex];
		if (TranslationUnit != nullptr)
		{
			// If HasBeenReviewed is false, this is a suggestion translation from a previous translation for the same Namespace/Key pair
			if (!TranslationUnit->HasBeenReviewed)
			{
				if (!Untranslated.Contains(TranslationUnit))
				{
					Untranslated.Add(TranslationUnit);
				}
				if (Review.Contains(TranslationUnit))
				{
					Review.Remove(TranslationUnit);
				}

				// Erase previously suggested translation from history (it has not been reviewed)
				TranslationUnit->Translation.Empty();

				// Remove all history entries
				for (FTranslationContextInfo Context : TranslationUnit->Contexts)
				{
					Context.Changes.Empty();
				}
			}
		}
	}
}

bool FTranslationDataManager::SaveSelectedTranslations(TArray<UTranslationUnit*> TranslationUnitsToSave, bool bSaveChangesToTranslationService)
{
	bool bSucceeded = true;
	
	TMap<FString, TSharedPtr<TArray<UTranslationUnit*>>> TextsToSavePerProject;

	// Regroup the translations to save by project
	for (UTranslationUnit* TextToSave : TranslationUnitsToSave)
	{
		FString LocresFilePath = TextToSave->LocresPath;
		if (!LocresFilePath.IsEmpty())
		{
			if (!TextsToSavePerProject.Contains(LocresFilePath))
			{
				TextsToSavePerProject.Add(LocresFilePath, MakeShareable(new TArray<UTranslationUnit*>()));
			}

			TSharedPtr<TArray<UTranslationUnit*>> ProjectArray = TextsToSavePerProject.FindRef(LocresFilePath);
			ProjectArray->Add(TextToSave);
		}
	}

	for (auto TextIt = TextsToSavePerProject.CreateIterator(); TextIt; ++TextIt)
	{
		auto Item = *TextIt;
		FString CurrentLocResPath = Item.Key;
		FString ManifestAndArchiveName = FPaths::GetBaseFilename(CurrentLocResPath);

		FString ArchiveFilePath = FPaths::GetPath(CurrentLocResPath);
		FString CultureName = FPaths::GetBaseFilename(ArchiveFilePath);
		FString ManifestPath = FPaths::GetPath(ArchiveFilePath);
		FString ArchiveFullPath = ArchiveFilePath / ManifestAndArchiveName + ".archive";
		FString ManifestFullPath = ManifestPath / ManifestAndArchiveName + ".manifest";
		FString EngineFullPath = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir());
		bool IsEngineManifest = false;
		if (ManifestFullPath.StartsWith(EngineFullPath))
		{
			IsEngineManifest = true;
		}

		ULocalizationTarget* LocalizationTarget = ILocalizationModule::Get().GetLocalizationTargetByName(ManifestAndArchiveName, IsEngineManifest);

		if (LocalizationTarget && FPaths::FileExists(ManifestFullPath) && FPaths::FileExists(ArchiveFullPath))
		{
			FString NativeCultureName;
			if (LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
			{
				NativeCultureName = LocalizationTarget->Settings.SupportedCulturesStatistics[LocalizationTarget->Settings.NativeCultureIndex].CultureName;
			}
			FString NativeArchiveFullPath = ManifestPath / NativeCultureName / ManifestAndArchiveName + ".archive";

			TSharedRef<FTranslationDataManager> DataManager = MakeShareable(new FTranslationDataManager(ManifestFullPath, NativeArchiveFullPath, ArchiveFullPath));

			if (DataManager->GetLoadedSuccessfully())
			{
				FPortableObjectFormatDOM PortableObjectDom;
				PortableObjectDom.SetProjectName(ManifestAndArchiveName);
				PortableObjectDom.SetLanguage(CultureName);
				PortableObjectDom.CreateNewHeader(GetDefault<UGeneralProjectSettings>()->CopyrightNotice);
				PortableObjectPipeline::UpdatePOFileHeaderForSettings(PortableObjectDom, LocalizationTarget->Settings.ExportSettings.CollapseMode, LocalizationTarget->Settings.ExportSettings.POFormat);

				TArray<UTranslationUnit*>& TranslationsArray = DataManager->GetAllTranslationsArray();
				TSharedPtr<TArray<UTranslationUnit*>> EditedItems = Item.Value;

				// For each edited item belonging to this manifest/archive pair
				for (auto EditedItemIt = EditedItems->CreateIterator(); EditedItemIt; ++EditedItemIt)
				{
					UTranslationUnit* EditedItem = *EditedItemIt;

					// Search all translations for the one that matches this FText
					for (UTranslationUnit* Translation : TranslationsArray)
					{
						bool bFoundMatchingTranslation = false;

						// If namespace matches...
						if (Translation->Namespace.Equals(EditedItem->Namespace))
						{
							// And key matches...
							for (const FTranslationContextInfo& ContextInfo : Translation->Contexts)
							{
								if (ContextInfo.Key.Equals(EditedItem->Key))
								{
									bFoundMatchingTranslation = true;

									// Update the translation in TranslationDataManager, and finish searching these translations
									Translation->Translation = EditedItem->Translation;

									// Add the PO entry
									{
										TSharedRef<FPortableObjectEntry> PoEntry = MakeShareable(new FPortableObjectEntry());
										PortableObjectPipeline::PopulateBasicPOFileEntry(*PoEntry, Translation->Namespace, ContextInfo.Key, nullptr, Translation->Source, Translation->Translation, LocalizationTarget->Settings.ExportSettings.CollapseMode, LocalizationTarget->Settings.ExportSettings.POFormat);

										//@TODO: We support additional metadata entries that can be translated.  How do those fit in the PO file format?  Ex: isMature
										PoEntry->AddReference(ContextInfo.Context); // Source location.

										PoEntry->AddExtractedComment(PortableObjectPipeline::GetConditionedKeyForExtractedComment(ContextInfo.Key)); // "Notes from Programmer" in the form of the Key.
										PoEntry->AddExtractedComment(PortableObjectPipeline::GetConditionedReferenceForExtractedComment(ContextInfo.Context)); // "Notes from Programmer" in the form of the Source Location, since this comes in handy too and OneSky doesn't properly show references, only comments.
#if WITH_EDITORONLY_DATA
										PoEntry->AddExtractedComment(PortableObjectPipeline::GetConditionedDevNotesForExtractedComment(ContextInfo.DevNotes));
#endif
										PortableObjectDom.AddEntry(PoEntry);
									}
									break;
								}
							}

							if (bFoundMatchingTranslation)
							{
								break;
							}
						}
					}
				}

				if (bSaveChangesToTranslationService)
				{
					FString UploadFilePath = FPaths::ProjectSavedDir() / "Temp" / CultureName / ManifestAndArchiveName + ".po";
					FFileHelper::SaveStringToFile(PortableObjectDom.ToString(), *UploadFilePath);

					FGuid LocalizationTargetGuid = LocalizationTarget->Settings.Guid;

					ILocalizationServiceProvider& Provider = ILocalizationServiceModule::Get().GetProvider();
					TSharedRef<FUploadLocalizationTargetFile, ESPMode::ThreadSafe> UploadTargetFileOp = ILocalizationServiceOperation::Create<FUploadLocalizationTargetFile>();
					UploadTargetFileOp->SetInTargetGuid(LocalizationTargetGuid);
					UploadTargetFileOp->SetInLocale(CultureName);
					FPaths::MakePathRelativeTo(UploadFilePath, *FPaths::ProjectDir());
					UploadTargetFileOp->SetInRelativeInputFilePathAndName(UploadFilePath);
					UploadTargetFileOp->SetPreserveAllText(true);

					Provider.Execute(UploadTargetFileOp, TArray<FLocalizationServiceTranslationIdentifier>(), ELocalizationServiceOperationConcurrency::Asynchronous, FLocalizationServiceOperationComplete::CreateStatic(&FTranslationDataManager::SaveSelectedTranslationsToTranslationServiceCallback));
				}

			}
			else
			{
				bSucceeded = false;
			}

			// Save the data to file, and preview in editor
			bSucceeded = bSucceeded && DataManager->WriteTranslationData();
			DataManager->PreviewAllTranslationsInEditor(LocalizationTarget);
		}
		else
		{
			bSucceeded = false;
		}
	}

	return bSucceeded;
}

void FTranslationDataManager::SaveSelectedTranslationsToTranslationServiceCallback(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result)
{
	TSharedPtr<FUploadLocalizationTargetFile, ESPMode::ThreadSafe> UploadLocalizationTargetOp = StaticCastSharedRef<FUploadLocalizationTargetFile>(Operation);
	bool bError = !(Result == ELocalizationServiceOperationCommandResult::Succeeded);
	FText ErrorText = FText::GetEmpty();
	FGuid InTargetGuid;
	FString InLocale;
	FString InRelativeOutputFilePathAndName;
	FString TargetName = "";
	FString TargetPath = "";
	FString CultureName = "";
	if (UploadLocalizationTargetOp.IsValid())
	{
		ErrorText = UploadLocalizationTargetOp->GetOutErrorText();
		InTargetGuid = UploadLocalizationTargetOp->GetInTargetGuid();
		InLocale = UploadLocalizationTargetOp->GetInLocale();
		InRelativeOutputFilePathAndName = UploadLocalizationTargetOp->GetInRelativeInputFilePathAndName();
		TargetName = FPaths::GetBaseFilename(InRelativeOutputFilePathAndName);
		TargetPath = FPaths::GetPath(InRelativeOutputFilePathAndName);
		CultureName = FPaths::GetBaseFilename(TargetPath);
	}

	// Try to get display name
	FInternationalization& I18N = FInternationalization::Get();
	FCulturePtr CulturePtr = I18N.GetCulture(CultureName);
	FString CultureDisplayName = CultureName;
	if (CulturePtr.IsValid())
	{
		CultureName = CulturePtr->GetDisplayName();
	}

	if (!bError && ErrorText.IsEmpty())
	{
		FText SuccessText = FText::Format(LOCTEXT("SaveSelectedTranslationsToTranslationServiceSuccess", "{0} translations for {1} target uploaded for processing to Translation Service."), FText::FromString(CultureDisplayName), FText::FromString(TargetName));
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Info(SuccessText);
		TranslationEditorMessageLog.Notify(SuccessText, EMessageSeverity::Info, true);
	}
	else
	{
		if (ErrorText.IsEmpty())
		{
			ErrorText = LOCTEXT("SaveToLocalizationServiceUnspecifiedError", "An unspecified error occured when trying to save to the Localization Service.");
		}

		FText ErrorNotify = FText::Format(LOCTEXT("SaveSelectedTranslationsToTranslationServiceFail", "{0} translations for {1} target failed to save to Translation Service!"), FText::FromString(CultureDisplayName), FText::FromString(TargetName));
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Error(ErrorNotify);
		TranslationEditorMessageLog.Error(ErrorText);
		TranslationEditorMessageLog.Notify(ErrorNotify);
	}
}

#undef LOCTEXT_NAMESPACE
