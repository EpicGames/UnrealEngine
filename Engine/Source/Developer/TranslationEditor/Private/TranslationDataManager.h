// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "ILocalizationServiceProvider.h"

class FInternationalizationArchive;
class FInternationalizationManifest;
class FJsonObject;
class ULocalizationTarget;
class UTranslationUnit;

class FTranslationDataManager : public TSharedFromThis<FTranslationDataManager>
{

public:

	FTranslationDataManager(const FString& InManifestFilePath, const FString& InNativeArchiveFilePath, const FString& InArchiveFilePath);
	FTranslationDataManager(ULocalizationTarget* const LocalizationTarget, const FString& CulturetoEdit);

	virtual ~FTranslationDataManager();

	TArray<UTranslationUnit*>& GetAllTranslationsArray()
	{
		return AllTranslations;
	}

	TArray<UTranslationUnit*>& GetUntranslatedArray()
	{
		return Untranslated;
	}

	TArray<UTranslationUnit*>& GetReviewArray()
	{
		return Review;
	}

	TArray<UTranslationUnit*>& GetCompleteArray()
	{
		return Complete;
	}

	TArray<UTranslationUnit*>& GetSearchResultsArray()
	{
		return SearchResults;
	}

	TArray<UTranslationUnit*>& GetChangedOnImportArray()
	{
		return ChangedOnImport;
	}
	
	/** Write the translation data in memory out to .archive file (check out the .archive file first if necessary) 
	* @param bForceWrite Whether or not to force a write even if the data hasn't changed
	* @param return Whether or not the write succeeded (true is succeeded)
	*/
	bool WriteTranslationData(bool bForceWrite = false);

	/** Delegate called when a TranslationDataObject property is changed */
	void HandlePropertyChanged(FName PropertyName);

	/** Regenerate and reload archives to reflect modifications in the UI */
	void PreviewAllTranslationsInEditor(ULocalizationTarget* LocalizationTarget);

	/** Put items in the Search Array if they match this filter */
	void PopulateSearchResultsUsingFilter(const FString& SearchFilter);

	/** Load (or reload) Translations from Archive file */ 
	void LoadFromArchive(TArray<UTranslationUnit*>& InTranslationUnits, bool bTrackChanges = false, bool bReloadFromFile = false);

	/** Get the history data for all translation units */
	void GetHistoryForTranslationUnits();

	/** Unload History information (in the case we reload it) */
	void UnloadHistoryInformation();

	/** Save the specified translations */
	static bool SaveSelectedTranslations(TArray<UTranslationUnit*> TranslationUnitsToSave, bool bSaveChangesToTranslationService = false);

	/** Save the specified translations */
	static void SaveSelectedTranslationsToTranslationServiceCallback(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result);

	/** Whether or not we successfully loaded the .manifest and .archive */
	bool GetLoadedSuccessfully()
	{
		return bLoadedSuccessfully;
	}

private:
	void Initialize();

	/** Take a path and a manifest name and return a manifest data structure */
	TSharedPtr< FInternationalizationManifest > ReadManifest ( const FString& ManifestFilePath );
	/** Retrieve an archive data structure from ArchiveFilePath */
	TSharedPtr< FInternationalizationArchive > ReadArchive(const FString& ArchiveFilePath);

	/** Write JSON string to text file */
	bool WriteJSONToTextFile( const FString& Output, const FString& Filename );

	/** Removes each UTranslationUnit in the passed array from the root set, allowing it to be garbage collected */
	void RemoveTranslationUnitArrayfromRoot( TArray<UTranslationUnit*>& TranslationUnits );

	// Arrays containing the translation data
	TArray<UTranslationUnit*> AllTranslations;
	TArray<UTranslationUnit*> Untranslated;
	TArray<UTranslationUnit*> Review;
	TArray<UTranslationUnit*> Complete;
	TArray<UTranslationUnit*> SearchResults;
	TArray<UTranslationUnit*> ChangedOnImport;

	/** Archive for the current project and native language */
	TSharedPtr< FInternationalizationArchive > NativeArchivePtr;
	/** Archive for the current project and translation language */
	TSharedPtr< FInternationalizationArchive > ArchivePtr;
	/** Manifest for the current project */
	TSharedPtr< FInternationalizationManifest > ManifestAtHeadRevisionPtr;

	/** Name of the manifest file */
	FString ManifestName;
	/** Path to the project */
	FString ProjectPath;
	/** Name of the archive file */
	FString ArchiveName;
	/** Path to the culture (language, sort of) we are targeting */
	FString CulturePath;
	/** Path to the manifest file **/
	FString OpenedManifestFilePath;
	/** Path to the native culture's archive file **/
	FString NativeArchiveFilePath;
	/** Path to the archive file **/
	FString OpenedArchiveFilePath;

	/** Files that are already checked out from Perforce **/
	TArray<FString> CheckedOutFiles;

	/** The localization target associated with the files being used/edited, if any. */
	TWeakObjectPtr<ULocalizationTarget> AssociatedLocalizationTarget;

	/** Whether or not we successfully loaded the .manifest and .archive */
	bool bLoadedSuccessfully;
};
