// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR

#include "TranslationEditor.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "TranslationEditorModule.h"
#include "TranslationEditorMenu.h"
#include "PropertyEditorModule.h"
#include "PropertyPath.h"

#include "Logging/MessageLog.h"

#include "IPropertyTableRow.h"
#include "DesktopPlatformModule.h"
#include "IPropertyTableWidgetHandle.h"

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "ILocalizationServiceModule.h"
#include "LocalizationCommandletTasks.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "TranslationEditor"

namespace TranslationEditorUtils
{

/** Get the filename used by the given font info */
FString GetFontFilename(const FSlateFontInfo& InFontInfo)
{
	const FCompositeFont* const ResolvedCompositeFont = InFontInfo.GetCompositeFont();
	return (ResolvedCompositeFont && ResolvedCompositeFont->DefaultTypeface.Fonts.Num() > 0)
		? ResolvedCompositeFont->DefaultTypeface.Fonts[0].Font.GetFontFilename()
		: FString();
}

} // namespace TranslationEditorUtils

const FName FTranslationEditor::UntranslatedTabId( TEXT( "TranslationEditor_Untranslated" ) );
const FName FTranslationEditor::ReviewTabId( TEXT( "TranslationEditor_Review" ) );
const FName FTranslationEditor::CompletedTabId( TEXT( "TranslationEditor_Completed" ) );
const FName FTranslationEditor::PreviewTabId( TEXT( "TranslationEditor_Preview" ) );
const FName FTranslationEditor::ContextTabId( TEXT( "TranslationEditor_Context" ) );
const FName FTranslationEditor::HistoryTabId( TEXT( "TranslationEditor_History" ) );
const FName FTranslationEditor::SearchTabId( TEXT( "TranslationEditor_Search" ) );
const FName FTranslationEditor::ChangedOnImportTabId( TEXT( "TranslationEditor_ChangedOnImport" ) );

void FTranslationEditor::Initialize()
{
	// Set up delegate functions for the buttons/spinboxes in the custom font columns' headers
	SourceColumn->SetOnChangeFontButtonClicked(FOnClicked::CreateSP(this, &FTranslationEditor::ChangeSourceFont_FReply));
	SourceColumn->SetOnFontSizeValueCommitted(FOnInt32ValueCommitted::CreateSP(this, &FTranslationEditor::OnSourceFontSizeCommitt));
	TranslationColumn->SetOnChangeFontButtonClicked(FOnClicked::CreateSP(this, &FTranslationEditor::ChangeTranslationTargetFont_FReply));
	TranslationColumn->SetOnFontSizeValueCommitted(FOnInt32ValueCommitted::CreateSP(this, &FTranslationEditor::OnTranslationTargetFontSizeCommitt));
}

void FTranslationEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TranslationEditor", "Translation Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( UntranslatedTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_Untranslated) )
		.SetDisplayName( LOCTEXT("UntranslatedTab", "Untranslated") )
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( ReviewTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_Review) )
		.SetDisplayName( LOCTEXT("ReviewTab", "Needs Review") )
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( CompletedTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_Completed) )
		.SetDisplayName( LOCTEXT("CompletedTab", "Completed") )
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( PreviewTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_Preview) )
		.SetDisplayName( LOCTEXT("PreviewTab", "Preview") )
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( ContextTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_Context) )
		.SetDisplayName( LOCTEXT("ContextTab", "Context") )
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( HistoryTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_History) )
		.SetDisplayName( LOCTEXT("HistoryTab", "History") )
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( SearchTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_Search) )
		.SetDisplayName(LOCTEXT("SearchTab", "Search"))
		.SetGroup( WorkspaceMenuCategoryRef );

	InTabManager->RegisterTabSpawner( ChangedOnImportTabId, FOnSpawnTab::CreateSP(this, &FTranslationEditor::SpawnTab_ChangedOnImport) )
		.SetDisplayName(LOCTEXT("ChangedOnImportTab", "Changed On Import"))
		.SetGroup( WorkspaceMenuCategoryRef );
}

void FTranslationEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner( UntranslatedTabId );
	InTabManager->UnregisterTabSpawner( ReviewTabId );
	InTabManager->UnregisterTabSpawner( CompletedTabId );
	InTabManager->UnregisterTabSpawner( PreviewTabId );
	InTabManager->UnregisterTabSpawner( ContextTabId );
	InTabManager->UnregisterTabSpawner(HistoryTabId);
	InTabManager->UnregisterTabSpawner(SearchTabId);
	InTabManager->UnregisterTabSpawner(ChangedOnImportTabId);
}

void FTranslationEditor::InitTranslationEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost )
{	
	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_TranslationEditor_Layout_v2" )
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.5)
			->SetHideTabWell( false )
			->AddTab( UntranslatedTabId, ETabState::OpenedTab )
			->AddTab( ReviewTabId,  ETabState::OpenedTab )
			->AddTab( CompletedTabId,  ETabState::OpenedTab )
			->AddTab( SearchTabId, ETabState::ClosedTab )
			->AddTab( ChangedOnImportTabId, ETabState::ClosedTab )
		)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.5)
			->SetHideTabWell(false)
			->AddTab(PreviewTabId, ETabState::OpenedTab)
		)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(false)
				->AddTab(ContextTabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(false)
				->AddTab(HistoryTabId, ETabState::OpenedTab)
			)
		)
	);

	// Register the UI COMMANDS and map them to our functions
	MapActions();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	// Need editing object to not be null
	UTranslationUnit* EditingObject;
	if (DataManager->GetAllTranslationsArray().Num() > 0 && DataManager->GetAllTranslationsArray()[0] != NULL)
	{
		EditingObject = DataManager->GetAllTranslationsArray()[0];
	}
	else
	{
		EditingObject = NewObject<UTranslationUnit>();
	}
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FTranslationEditorModule::TranslationEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, EditingObject);
	
	FTranslationEditorModule& TranslationEditorModule = FModuleManager::LoadModuleChecked<FTranslationEditorModule>( "TranslationEditor" );
	AddMenuExtender(TranslationEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	TSharedPtr<FExtender> MenuExtender = MakeShareable(new FExtender);
	FTranslationEditorMenu::SetupTranslationEditorMenu( MenuExtender, *this );
	AddMenuExtender(MenuExtender);

	AddToolbarExtender(TranslationEditorModule.GetToolbarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	FTranslationEditorMenu::SetupTranslationEditorToolbar( ToolbarExtender, *this );
	AddToolbarExtender(ToolbarExtender);

	RegenerateMenusAndToolbars();

	// @todo toolkit world centric editing
	/*// Setup our tool's layout
	if( IsWorldCentricAssetEditor() )
	{
		const FString TabInitializationPayload(TEXT(""));		// NOTE: Payload not currently used for table properties
		SpawnToolkitTab( UntranslatedTabId, TabInitializationPayload, EToolkitTabSpot::Details );
	}*/

	// NOTE: Could fill in asset editor commands here!
}

FName FTranslationEditor::GetToolkitFName() const
{
	return FName("TranslationEditor");
}

FText FTranslationEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Translation Editor" );
}

FText FTranslationEditor::GetToolkitName() const
{
	const UObject* EditingObject = GetEditingObject();
	check (EditingObject != NULL);

	FFormatNamedArguments Args;
	Args.Add(TEXT("Language"), FText::FromString(FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath))));
	Args.Add(TEXT("ProjectName"), FText::FromString(FPaths::GetBaseFilename(ManifestFilePath)));
	Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );
	return FText::Format( LOCTEXT("TranslationEditorAppLabel", "{Language} - {ProjectName} - {ToolkitName}"), Args );
}

FText FTranslationEditor::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetEditingObject();

	check (EditingObject != NULL);

	FFormatNamedArguments Args;
	Args.Add(TEXT("Language"), FText::FromString(FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath))));
	Args.Add(TEXT("ProjectName"), FText::FromString(FPaths::GetBaseFilename(ManifestFilePath)));
	Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );
	return FText::Format( LOCTEXT("TranslationEditorAppToolTip", "{Language} - {ProjectName} - {ToolkitName}"), Args );
}

FString FTranslationEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Translation ").ToString();
}

FLinearColor FTranslationEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}
FText FTranslationEditor::GetTabLabel_Untranslated() const
{
	const int32 UntranslatedNum = DataManager->GetUntranslatedArray().Num();
	return UntranslatedNum > 0 ?
		FText::Format(LOCTEXT("UntranslatedTabTitleNumbered", "Untranslated ({0})"), UntranslatedNum) :
		LOCTEXT("UntranslatedTabTitle", "Untranslated");
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_Untranslated( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == UntranslatedTabId );

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	FProperty* SourceProperty = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Source");
	FProperty* TranslationProperty = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Translation");

	// create empty property table
	UntranslatedPropertyTable = PropertyEditorModule.CreatePropertyTable();
	UntranslatedPropertyTable->SetIsUserAllowedToChangeRoot( false );
	UntranslatedPropertyTable->SetOrientation( EPropertyTableOrientation::AlignPropertiesInColumns );
	UntranslatedPropertyTable->SetShowRowHeader( true );
	UntranslatedPropertyTable->SetShowObjectName( false );
	UntranslatedPropertyTable->OnSelectionChanged()->AddSP( this, &FTranslationEditor::UpdateUntranslatedSelection );

	// we want to customize some columns
	TArray< TSharedRef<class IPropertyTableCustomColumn>> CustomColumns;
	SourceColumn->AddSupportedProperty(SourceProperty);
	TranslationColumn->AddSupportedProperty(TranslationProperty);
	CustomColumns.Add( SourceColumn );
	CustomColumns.Add(TranslationColumn);

	UntranslatedPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetUntranslatedArray());

	// Add the columns we want to display
	UntranslatedPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)SourceProperty);
	UntranslatedPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)TranslationProperty);

	// Freeze columns, don't want user to remove them
	TArray<TSharedRef<IPropertyTableColumn>> Columns = UntranslatedPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	UntranslatedPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle( UntranslatedPropertyTable.ToSharedRef(), CustomColumns);
	TSharedRef<SWidget> PropertyTableWidget = UntranslatedPropertyTableWidgetHandle->GetWidget();

	

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon( FAppStyle::GetBrush("TranslationEditor.Tabs.Properties") )
		.Label(this, &FTranslationEditor::GetTabLabel_Untranslated)
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Padding(0.0f)
			[
				PropertyTableWidget
			]
		];

	UntranslatedTab = NewDockTab;

	return NewDockTab;
}

FText FTranslationEditor::GetTabLabel_Review() const
{
	const int32 ReviewNum = DataManager->GetReviewArray().Num();
	return ReviewNum > 0 ?
		FText::Format(LOCTEXT("ReviewTabTitleNumbered", "Needs Review ({0})"), ReviewNum) :
		LOCTEXT("ReviewTabTitle", "Needs Review");
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_Review( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == ReviewTabId );

	FProperty* SourceProperty = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Source");
	FProperty* TranslationProperty = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Translation");

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	// create empty property table
	ReviewPropertyTable = PropertyEditorModule.CreatePropertyTable();
	ReviewPropertyTable->SetIsUserAllowedToChangeRoot( false );
	ReviewPropertyTable->SetOrientation( EPropertyTableOrientation::AlignPropertiesInColumns );
	ReviewPropertyTable->SetShowRowHeader( true );
	ReviewPropertyTable->SetShowObjectName( false );
	ReviewPropertyTable->OnSelectionChanged()->AddSP( this, &FTranslationEditor::UpdateNeedsReviewSelection );

	// we want to customize some columns
	TArray< TSharedRef< class IPropertyTableCustomColumn > > CustomColumns;
	SourceColumn->AddSupportedProperty(SourceProperty);
	TranslationColumn->AddSupportedProperty(TranslationProperty);
	CustomColumns.Add( SourceColumn );
	CustomColumns.Add( TranslationColumn );

	ReviewPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetReviewArray());

	// Add the columns we want to display
	ReviewPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Source"));
	ReviewPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Translation"));
	ReviewPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "HasBeenReviewed"));

	TArray<TSharedRef<IPropertyTableColumn>> Columns = ReviewPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		FString ColumnId = Column->GetId().ToString();
		if (ColumnId == "HasBeenReviewed")
		{
			Column->SetWidth(120);
			Column->SetSizeMode(EPropertyTableColumnSizeMode::Fixed);
		}
		// Freeze columns, don't want user to remove them
		Column->SetFrozen(true);
	}

	ReviewPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle( ReviewPropertyTable.ToSharedRef(), CustomColumns);
	TSharedRef<SWidget> PropertyTableWidget = ReviewPropertyTableWidgetHandle->GetWidget();

	

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon( FAppStyle::GetBrush("TranslationEditor.Tabs.Properties") )
		.Label(this, &FTranslationEditor::GetTabLabel_Review)
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Padding(0.0f)
			[
				PropertyTableWidget
			]
		];

	ReviewTab = NewDockTab;

	return NewDockTab;
}

FText FTranslationEditor::GetTabLabel_Completed() const
{
	const int32 CompletedNum = DataManager->GetCompleteArray().Num();
	return CompletedNum > 0 ?
		FText::Format(LOCTEXT("CompletedTabTitleNumbered", "Completed ({0})"), CompletedNum) :
		LOCTEXT("CompletedTabTitle", "Completed");
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_Completed( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == CompletedTabId );

	FProperty* SourceProperty = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Source");
	FProperty* TranslationProperty = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Translation");

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	// create empty property table
	CompletedPropertyTable = PropertyEditorModule.CreatePropertyTable();
	CompletedPropertyTable->SetIsUserAllowedToChangeRoot( false );
	CompletedPropertyTable->SetOrientation( EPropertyTableOrientation::AlignPropertiesInColumns );
	CompletedPropertyTable->SetShowRowHeader( true );
	CompletedPropertyTable->SetShowObjectName( false );
	CompletedPropertyTable->OnSelectionChanged()->AddSP( this, &FTranslationEditor::UpdateCompletedSelection );

	// we want to customize some columns
	TArray< TSharedRef< class IPropertyTableCustomColumn > > CustomColumns;
	SourceColumn->AddSupportedProperty(SourceProperty);
	TranslationColumn->AddSupportedProperty(TranslationProperty);
	CustomColumns.Add( SourceColumn );
	CustomColumns.Add( TranslationColumn );

	CompletedPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetCompleteArray());

	// Add the columns we want to display
	CompletedPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Source"));
	CompletedPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Translation"));

	// Freeze columns, don't want user to remove them
	TArray<TSharedRef<IPropertyTableColumn>> Columns = CompletedPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	CompletedPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle( CompletedPropertyTable.ToSharedRef(), CustomColumns);
	TSharedRef<SWidget> PropertyTableWidget = CompletedPropertyTableWidgetHandle->GetWidget();

	

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon( FAppStyle::GetBrush("TranslationEditor.Tabs.Properties") )
		.Label(this, &FTranslationEditor::GetTabLabel_Completed)
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Padding(0.0f)
			[
				PropertyTableWidget
			]
		];

	CompletedTab = NewDockTab;

	return NewDockTab;
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_Search(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == SearchTabId);

	FProperty* SourceProperty = FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source");
	FProperty* TranslationProperty = FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation");

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// create empty property table
	SearchPropertyTable = PropertyEditorModule.CreatePropertyTable();
	SearchPropertyTable->SetIsUserAllowedToChangeRoot(false);
	SearchPropertyTable->SetOrientation(EPropertyTableOrientation::AlignPropertiesInColumns);
	SearchPropertyTable->SetShowRowHeader(true);
	SearchPropertyTable->SetShowObjectName(false);
	SearchPropertyTable->OnSelectionChanged()->AddSP(this, &FTranslationEditor::UpdateSearchSelection);

	// we want to customize some columns
	TArray< TSharedRef< class IPropertyTableCustomColumn > > CustomColumns;
	SourceColumn->AddSupportedProperty(SourceProperty);
	TranslationColumn->AddSupportedProperty(TranslationProperty);
	CustomColumns.Add(SourceColumn);
	CustomColumns.Add(TranslationColumn);

	SearchPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetSearchResultsArray());

	// Add the columns we want to display
	SearchPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source"));
	SearchPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation"));

	// Freeze columns, don't want user to remove them
	TArray<TSharedRef<IPropertyTableColumn>> Columns = SearchPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	SearchPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle(SearchPropertyTable.ToSharedRef(), CustomColumns);
	TSharedRef<SWidget> PropertyTableWidget = SearchPropertyTableWidgetHandle->GetWidget();

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon(FAppStyle::GetBrush("TranslationEditor.Tabs.Properties"))
		.Label(LOCTEXT("SearchTabTitle", "Search"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("FilterSearch", "Search..."))
				.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search"))
				.OnTextChanged(this, &FTranslationEditor::OnFilterTextChanged)
				.OnTextCommitted(this, &FTranslationEditor::OnFilterTextCommitted)
			]
			+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.FillHeight(10.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				.VAlign(VAlign_Top)
				[
					PropertyTableWidget
				]
			]
		];

	SearchTab = NewDockTab;

	return NewDockTab;
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_ChangedOnImport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ChangedOnImportTabId);

	FProperty* SourceProperty = FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source");
	FProperty* TranslationBeforeImportProperty = FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "TranslationBeforeImport");
	FProperty* TranslationProperty = FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation");

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// create empty property table
	ChangedOnImportPropertyTable = PropertyEditorModule.CreatePropertyTable();
	ChangedOnImportPropertyTable->SetIsUserAllowedToChangeRoot(false);
	ChangedOnImportPropertyTable->SetOrientation(EPropertyTableOrientation::AlignPropertiesInColumns);
	ChangedOnImportPropertyTable->SetShowRowHeader(true);
	ChangedOnImportPropertyTable->SetShowObjectName(false);
	ChangedOnImportPropertyTable->OnSelectionChanged()->AddSP(this, &FTranslationEditor::UpdateSearchSelection);

	// we want to customize some columns
	TArray< TSharedRef< class IPropertyTableCustomColumn > > CustomColumns;
	SourceColumn->AddSupportedProperty(SourceProperty);
	TranslationColumn->AddSupportedProperty(TranslationProperty);
	CustomColumns.Add(SourceColumn);
	CustomColumns.Add(TranslationColumn);

	ChangedOnImportPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetSearchResultsArray());

	// Add the columns we want to display
	ChangedOnImportPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source"));
	ChangedOnImportPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "TranslationBeforeImport"));
	ChangedOnImportPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation"));

	// Freeze columns, don't want user to remove them
	TArray<TSharedRef<IPropertyTableColumn>> Columns = ChangedOnImportPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	SearchPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle(ChangedOnImportPropertyTable.ToSharedRef(), CustomColumns);
	TSharedRef<SWidget> PropertyTableWidget = SearchPropertyTableWidgetHandle->GetWidget();

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon(FAppStyle::GetBrush("TranslationEditor.Tabs.Properties"))
		.Label(LOCTEXT("ChangedOnImportTabTitle", "Changed on Import"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				PropertyTableWidget
			]
		];

	ChangedOnImportTab = NewDockTab;

	return NewDockTab;
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_Preview( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == PreviewTabId );

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon( FAppStyle::GetBrush("TranslationEditor.Tabs.Properties") )
		.Label( LOCTEXT("PreviewTabTitle", "Preview") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Padding(0.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					PreviewTextBlock
				]
			]
		];
	
	return NewDockTab;
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_Context( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == ContextTabId );

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	// create empty property table
	ContextPropertyTable = PropertyEditorModule.CreatePropertyTable();
	ContextPropertyTable->SetIsUserAllowedToChangeRoot( false );
	ContextPropertyTable->SetOrientation( EPropertyTableOrientation::AlignPropertiesInColumns );
	ContextPropertyTable->SetShowRowHeader( true );
	ContextPropertyTable->SetShowObjectName( false );
	ContextPropertyTable->OnSelectionChanged()->AddSP( this, &FTranslationEditor::UpdateContextSelection );

	if (DataManager->GetAllTranslationsArray().Num() > 0)
	{
		TArray<UObject*> Objects;
		Objects.Add(DataManager->GetAllTranslationsArray()[0]);
		ContextPropertyTable->SetObjects(Objects);
	}

	// Build the Path to the data we want to show
	FProperty* ContextProp = FindFProperty<FProperty>( UTranslationUnit::StaticClass(), "Contexts" );
	FPropertyInfo ContextPropInfo;
	ContextPropInfo.Property = ContextProp;
	ContextPropInfo.ArrayIndex = INDEX_NONE;
	TSharedRef<FPropertyPath> Path = FPropertyPath::CreateEmpty();
	Path = Path->ExtendPath(ContextPropInfo);
	ContextPropertyTable->SetRootPath(Path);

	// Add the columns we want to display
	ContextPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( FTranslationContextInfo::StaticStruct(), "Key"));
	ContextPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>( FTranslationContextInfo::StaticStruct(), "Context"));

	// Freeze columns, don't want user to remove them
	TArray<TSharedRef<IPropertyTableColumn>> Columns = ContextPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	ContextPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle( ContextPropertyTable.ToSharedRef() );
	TSharedRef<SWidget> PropertyTableWidget = ContextPropertyTableWidgetHandle->GetWidget();

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon( FAppStyle::GetBrush("TranslationEditor.Tabs.Properties") )
		.Label( LOCTEXT("ContextTabTitle", "Context") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Padding(0.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(5.0f)
					[
						NamespaceTextBlock
					]
				]
				+SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					PropertyTableWidget
				]
			]
		];
	
	return NewDockTab;
}

TSharedRef<SDockTab> FTranslationEditor::SpawnTab_History(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == HistoryTabId);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FProperty* SourceProperty = FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "Source");
	FProperty* TranslationProperty = FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "Translation");

	// create empty property table
	HistoryPropertyTable = PropertyEditorModule.CreatePropertyTable();
	HistoryPropertyTable->SetIsUserAllowedToChangeRoot(false);
	HistoryPropertyTable->SetOrientation(EPropertyTableOrientation::AlignPropertiesInColumns);
	HistoryPropertyTable->SetShowRowHeader(true);
	HistoryPropertyTable->SetShowObjectName(false);

	// we want to customize some columns
	TArray< TSharedRef<class IPropertyTableCustomColumn>> CustomColumns;
	SourceColumn->AddSupportedProperty(SourceProperty);
	TranslationColumn->AddSupportedProperty(TranslationProperty);
	CustomColumns.Add(SourceColumn);
	CustomColumns.Add(TranslationColumn);

	if (DataManager->GetAllTranslationsArray().Num() > 0)
	{
		TArray<UObject*> Objects;
		Objects.Add(DataManager->GetAllTranslationsArray()[0]);
		HistoryPropertyTable->SetObjects(Objects);
	}

	// Build the Path to the data we want to show
	TSharedRef<FPropertyPath> Path = FPropertyPath::CreateEmpty();
	FArrayProperty* ContextsProp = FindFProperty<FArrayProperty>(UTranslationUnit::StaticClass(), "Contexts");
	Path = Path->ExtendPath(FPropertyPath::Create(ContextsProp));
	FPropertyInfo ContextsPropInfo;
	ContextsPropInfo.Property = ContextsProp->Inner;
	ContextsPropInfo.ArrayIndex = 0;
	Path = Path->ExtendPath(ContextsPropInfo);

	FProperty* ChangesProp = FindFProperty<FProperty>(FTranslationContextInfo::StaticStruct(), "Changes");
	FPropertyInfo ChangesPropInfo;
	ChangesPropInfo.Property = ChangesProp;
	ChangesPropInfo.ArrayIndex = INDEX_NONE;
	Path = Path->ExtendPath(ChangesPropInfo);
	HistoryPropertyTable->SetRootPath(Path);

	// Add the columns we want to display
	HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "Version"));
	HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "DateAndTime"));
	HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)SourceProperty);
	HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)TranslationProperty);

	// Freeze columns, don't want user to remove them
	TArray<TSharedRef<IPropertyTableColumn>> Columns = HistoryPropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	HistoryPropertyTableWidgetHandle = PropertyEditorModule.CreatePropertyTableWidgetHandle(HistoryPropertyTable.ToSharedRef(), CustomColumns);
	TSharedRef<SWidget> PropertyTableWidget = HistoryPropertyTableWidgetHandle->GetWidget();

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		//.Icon(FAppStyle::GetBrush("TranslationEditor.Tabs.Properties"))
		.Label(LOCTEXT("HistoryTabTitle", "History"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(5.0f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(FOnClicked::CreateSP(this, &FTranslationEditor::OnGetHistoryButtonClicked))
						[
							SNew(STextBlock)
							.Text((LOCTEXT("GetHistoryButton", "Get History...")))
						]
					]
				]
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					PropertyTableWidget
				]
			]
		];
	
	return NewDockTab;
}

void FTranslationEditor::MapActions()
{
	FTranslationEditorCommands::Register();

	ToolkitCommands->MapAction( FTranslationEditorCommands::Get().ChangeSourceFont,
		FExecuteAction::CreateSP(this, &FTranslationEditor::ChangeSourceFont),
		FCanExecuteAction());

	ToolkitCommands->MapAction( FTranslationEditorCommands::Get().ChangeTranslationTargetFont,
		FExecuteAction::CreateSP(this, &FTranslationEditor::ChangeTranslationTargetFont),
		FCanExecuteAction());

	ToolkitCommands->MapAction( FTranslationEditorCommands::Get().SaveTranslations,
		FExecuteAction::CreateSP(this, &FTranslationEditor::SaveAsset_Execute),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FTranslationEditorCommands::Get().PreviewAllTranslationsInEditor,
		FExecuteAction::CreateSP(this, &FTranslationEditor::PreviewAllTranslationsInEditor_Execute),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FTranslationEditorCommands::Get().ImportLatestFromLocalizationService,
		FExecuteAction::CreateSP(this, &FTranslationEditor::ImportLatestFromLocalizationService_Execute),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FTranslationEditorCommands::Get().ExportToPortableObjectFormat,
		FExecuteAction::CreateSP(this, &FTranslationEditor::ExportToPortableObjectFormat_Execute),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FTranslationEditorCommands::Get().ImportFromPortableObjectFormat,
		FExecuteAction::CreateSP(this, &FTranslationEditor::ImportFromPortableObjectFormat_Execute),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FTranslationEditorCommands::Get().OpenSearchTab,
		FExecuteAction::CreateSP(this, &FTranslationEditor::OpenSearchTab_Execute),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FTranslationEditorCommands::Get().OpenTranslationPicker,
		FExecuteAction::CreateStatic(&ITranslationEditor::OpenTranslationPicker),
		FCanExecuteAction());
}

void FTranslationEditor::ChangeSourceFont()
{
	// Use path from current font
	FString DefaultFile = TranslationEditorUtils::GetFontFilename(SourceFont);

	FString NewFontFilename;
	bool bOpened = OpenFontPicker(DefaultFile, NewFontFilename);

	if ( bOpened && NewFontFilename != "")
	{
		TSharedRef<FCompositeFont> SourceCompositeFont = MakeShared<FStandaloneCompositeFont>();
		SourceCompositeFont->DefaultTypeface.AppendFont(TEXT("Regular"), NewFontFilename, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
		SourceFont = FSlateFontInfo(SourceCompositeFont, SourceFont.Size);
		RefreshUI();
	}
}

void FTranslationEditor::ChangeTranslationTargetFont()
{
	// Use path from current font
	FString DefaultFile = TranslationEditorUtils::GetFontFilename(TranslationTargetFont);

	FString NewFontFilename;
	bool bOpened = OpenFontPicker(DefaultFile, NewFontFilename);

	if ( bOpened && NewFontFilename != "")
	{
		TSharedRef<FCompositeFont> TargetCompositeFont = MakeShared<FStandaloneCompositeFont>();
		TargetCompositeFont->DefaultTypeface.AppendFont(TEXT("Regular"), NewFontFilename, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
		TranslationTargetFont = FSlateFontInfo(TargetCompositeFont, TranslationTargetFont.Size);
		RefreshUI();
	}
}

void FTranslationEditor::RefreshUI()
{
	// Set the fonts in our custom font columns and text block
	SourceColumn->SetFont(SourceFont);
	TranslationColumn->SetFont(TranslationTargetFont);
	PreviewTextBlock->SetFont(TranslationTargetFont);

	// Refresh our widget displays
	if (UntranslatedPropertyTableWidgetHandle.IsValid())
	{
		UntranslatedPropertyTableWidgetHandle->RequestRefresh();
	}
	if (ReviewPropertyTableWidgetHandle.IsValid())	
	{
		ReviewPropertyTableWidgetHandle->RequestRefresh();
	}
	if (CompletedPropertyTableWidgetHandle.IsValid())
	{
		CompletedPropertyTableWidgetHandle->RequestRefresh();
	}
	if (ContextPropertyTableWidgetHandle.IsValid())
	{
		ContextPropertyTableWidgetHandle->RequestRefresh();
	}
	if (HistoryPropertyTableWidgetHandle.IsValid())
	{
		HistoryPropertyTableWidgetHandle->RequestRefresh();
	}
	if (SearchPropertyTableWidgetHandle.IsValid())
	{
		SearchPropertyTableWidgetHandle->RequestRefresh();
	}
	if (ChangedOnImportPropertyTableWidgetHandle.IsValid())
	{
		ChangedOnImportPropertyTableWidgetHandle->RequestRefresh();
	}
}

bool FTranslationEditor::OpenFontPicker( const FString DefaultFile, FString& OutFile )
{
	const FString FontFileDescription = LOCTEXT( "FontFileDescription", "Font File" ).ToString();
	const FString FontFileExtension = TEXT("*.ttf;*.otf");
	const FString FileTypes = FString::Printf( TEXT("%s (%s)|%s"), *FontFileDescription, *FontFileExtension, *FontFileExtension );

	// Prompt the user for the filenames
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if ( DesktopPlatform )
	{
		void* ParentWindowWindowHandle = NULL;

		const TSharedPtr<SWindow>& ParentWindow = FSlateApplication::Get().FindWidgetWindow(PreviewTextBlock);
		if ( ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid() )
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("ChooseFontWindowTitle", "Choose Font").ToString(),
			FPaths::GetPath(DefaultFile),
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames
			);
	}

	if ( bOpened && OpenFilenames.Num() > 0 )
	{
		OutFile = OpenFilenames[0];
	} 
	else
	{
		OutFile = "";
	}

	return bOpened;
}

void FTranslationEditor::UpdateUntranslatedSelection()
{
	TSharedPtr<SDockTab> UntranslatedTabSharedPtr = UntranslatedTab.Pin();
	if (UntranslatedTabSharedPtr.IsValid() && UntranslatedTabSharedPtr->IsForeground() && UntranslatedPropertyTable.IsValid())
	{
		TSet<TSharedRef<IPropertyTableRow>> SelectedRows = UntranslatedPropertyTable->GetSelectedRows();
		UpdateTranslationUnitSelection(SelectedRows);
	}
}

void FTranslationEditor::UpdateNeedsReviewSelection()
{
	TSharedPtr<SDockTab> ReviewTabSharedPtr = ReviewTab.Pin();
	if (ReviewTabSharedPtr.IsValid() && ReviewTabSharedPtr->IsForeground() && ReviewPropertyTable.IsValid())
	{
		TSet<TSharedRef<IPropertyTableRow>> SelectedRows = ReviewPropertyTable->GetSelectedRows();
		UpdateTranslationUnitSelection(SelectedRows);
	}
}

void FTranslationEditor::UpdateCompletedSelection()
{
	TSharedPtr<SDockTab> CompletedTabSharedPtr = CompletedTab.Pin();
	if (CompletedTabSharedPtr.IsValid() && CompletedTabSharedPtr->IsForeground() && CompletedPropertyTable.IsValid())
	{
		TSet<TSharedRef<IPropertyTableRow>> SelectedRows = CompletedPropertyTable->GetSelectedRows();
		UpdateTranslationUnitSelection(SelectedRows);
	}
}

void FTranslationEditor::UpdateSearchSelection()
{
	TSharedPtr<SDockTab> SearchTabSharedPtr = SearchTab.Pin();
	if (SearchTabSharedPtr.IsValid() && SearchTabSharedPtr->IsForeground() && SearchPropertyTable.IsValid())
	{
		TSet<TSharedRef<IPropertyTableRow>> SelectedRows = SearchPropertyTable->GetSelectedRows();
		UpdateTranslationUnitSelection(SelectedRows);
	}
}

void FTranslationEditor::UpdateChangedOnImportSelection()
{
	TSharedPtr<SDockTab> ChangedOnImportTabSharedPtr = SearchTab.Pin();
	if (ChangedOnImportTabSharedPtr.IsValid() && ChangedOnImportTabSharedPtr->IsForeground() && ChangedOnImportPropertyTable.IsValid())
	{
		TSet<TSharedRef<IPropertyTableRow>> SelectedRows = ChangedOnImportPropertyTable->GetSelectedRows();
		UpdateTranslationUnitSelection(SelectedRows);
	}
}

void FTranslationEditor::UpdateTranslationUnitSelection(TSet<TSharedRef<IPropertyTableRow>>& SelectedRows)
{
	// Can only really handle single selection
	if (SelectedRows.Num() == 1)
	{
		TSharedRef<IPropertyTableRow> SelectedRow = *(SelectedRows.CreateConstIterator());
		TSharedRef<FPropertyPath> PartialPath = SelectedRow->GetPartialPath();

		TWeakObjectPtr<UObject> UObjectWeakPtr = SelectedRow->GetDataSource()->AsUObject();
		if (UObjectWeakPtr.IsValid())
		{
			UObject* UObjectPtr = UObjectWeakPtr.Get();
			if (UObjectPtr != NULL)
			{
				UTranslationUnit* SelectedTranslationUnit = (UTranslationUnit*)UObjectPtr;
				if (SelectedTranslationUnit != NULL)
				{
					PreviewTextBlock->SetText(FText::AsCultureInvariant(SelectedTranslationUnit->Translation));
					NamespaceTextBlock->SetText(FText::Format(LOCTEXT("TranslationNamespace", "Namespace: {0}"), FText::FromString(SelectedTranslationUnit->Namespace)));

					// Add the ContextPropertyTable-specific path
					FArrayProperty* ContextArrayProp = FindFProperty<FArrayProperty>(UTranslationUnit::StaticClass(), "Contexts");
					FPropertyInfo ContextArrayPropInfo;
					ContextArrayPropInfo.Property = ContextArrayProp;
					ContextArrayPropInfo.ArrayIndex = INDEX_NONE;
					TSharedRef<FPropertyPath> ContextPath = FPropertyPath::CreateEmpty();
					ContextPath = ContextPath->ExtendPath(PartialPath);
					ContextPath = ContextPath->ExtendPath(ContextArrayPropInfo);

					if (ContextPropertyTable.IsValid())
					{
						TArray<UObject*> ObjectArray;
						ObjectArray.Add(SelectedTranslationUnit);
						ContextPropertyTable->SetObjects(ObjectArray);
						ContextPropertyTable->SetRootPath(ContextPath);

						// Need to re-add the columns we want to display
						ContextPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationContextInfo::StaticStruct(), "Key"));
						ContextPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationContextInfo::StaticStruct(), "Context"));

						TArray<TSharedRef<IPropertyTableColumn>> Columns = ContextPropertyTable->GetColumns();
						for (TSharedRef<IPropertyTableColumn> Column : Columns)
						{
							Column->SetFrozen(true);
						}

						TSharedPtr<IPropertyTableCell> ContextToSelectPtr = ContextPropertyTable->GetFirstCellInTable();

						if (ContextToSelectPtr.IsValid())
						{
							TSet<TSharedRef<IPropertyTableCell>> CellsToSelect;
							CellsToSelect.Add(ContextToSelectPtr.ToSharedRef());
							ContextPropertyTable->SetSelectedCells(CellsToSelect);
						}
					}
				}
			}
		}
	}
}

void FTranslationEditor::SaveAsset_Execute()
{
	// Doesn't call parent SaveAsset_Execute, only need to tell data manager to write data
	DataManager->WriteTranslationData();
}

void FTranslationEditor::UpdateContextSelection()
{
	if (ContextPropertyTable.IsValid())
	{
		TSet<TSharedRef<IPropertyTableRow>> SelectedRows = ContextPropertyTable->GetSelectedRows();
		TSharedRef<FPropertyPath> InitialPath = ContextPropertyTable->GetRootPath();
		FProperty* PropertyToFind = InitialPath->GetRootProperty().Property.Get();

		// Can only really handle single selection
		if (SelectedRows.Num() == 1)
		{
			TSharedRef<IPropertyTableRow> SelectedRow = *(SelectedRows.CreateConstIterator());
			TSharedRef<FPropertyPath> PartialPath = SelectedRow->GetPartialPath();

			TWeakObjectPtr<UObject> UObjectWeakPtr = SelectedRow->GetDataSource()->AsUObject();
			if (UObjectWeakPtr.IsValid())
			{
				UObject* UObjectPtr = UObjectWeakPtr.Get();
				if (UObjectPtr != NULL)
				{
					UTranslationUnit* SelectedTranslationUnit = (UTranslationUnit*)UObjectPtr;
					if (SelectedTranslationUnit != NULL)
					{
						// Index of the leaf most property is the context info index we need
						FTranslationContextInfo& SelectedContextInfo = SelectedTranslationUnit->Contexts[PartialPath->GetLeafMostProperty().ArrayIndex];

						// If this is a translation unit from the review tab and they select a context, possibly update the selected translation with one from that context
						// Only change the suggested translation if they haven't yet reviewed it
						if (SelectedTranslationUnit->HasBeenReviewed == false)
						{
							for (int32 ChangeIndex = 0; ChangeIndex < SelectedContextInfo.Changes.Num(); ++ChangeIndex)
							{
								// Find most recent, non-empty translation
								if (!SelectedContextInfo.Changes[ChangeIndex].Translation.IsEmpty() && SelectedTranslationUnit->Translation != SelectedContextInfo.Changes[ChangeIndex].Translation)
								{
									SelectedTranslationUnit->Modify();
									SelectedTranslationUnit->Translation = SelectedContextInfo.Changes[ChangeIndex].Translation;
									SelectedTranslationUnit->PostEditChange();
								}
							}
						}


						// Add the HistoryPropertyTable-specific path
						TSharedRef<FPropertyPath> HistoryPath = ContextPropertyTable->GetRootPath();
						FArrayProperty* ContextArrayProp = FindFProperty<FArrayProperty>(UTranslationUnit::StaticClass(), "Contexts");
						FPropertyInfo ContextPropInfo;
						ContextPropInfo.Property = ContextArrayProp->Inner;
						ContextPropInfo.ArrayIndex = PartialPath->GetLeafMostProperty().ArrayIndex;
						HistoryPath = HistoryPath->ExtendPath(ContextPropInfo);
						FArrayProperty* ChangesProp = FindFProperty<FArrayProperty>(FTranslationContextInfo::StaticStruct(), "Changes");
						FPropertyInfo ChangesPropInfo;
						ChangesPropInfo.Property = ChangesProp;
						ChangesPropInfo.ArrayIndex = INDEX_NONE;
						HistoryPath = HistoryPath->ExtendPath(ChangesPropInfo);
						if (HistoryPropertyTable.IsValid())
						{
							TArray<UObject*> ObjectArray;
							ObjectArray.Add(SelectedTranslationUnit);
							HistoryPropertyTable->SetObjects(ObjectArray);
							HistoryPropertyTable->SetRootPath(HistoryPath);

							// Need to re-add the columns we want to display
							HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "Version"));
							HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "DateAndTime"));
							HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "Source"));
							HistoryPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(FTranslationChange::StaticStruct(), "Translation"));

							TArray<TSharedRef<IPropertyTableColumn>> Columns = HistoryPropertyTable->GetColumns();
							for (TSharedRef<IPropertyTableColumn> Column : Columns)
							{
								Column->SetFrozen(true);
							}
						}
					}
				}
			}
		}
	}
}

void FTranslationEditor::PreviewAllTranslationsInEditor_Execute()
{
	DataManager->PreviewAllTranslationsInEditor(AssociatedLocalizationTarget.Get());
}

void FTranslationEditor::ImportLatestFromLocalizationService_Execute()
{
	check(AssociatedLocalizationTarget.IsValid());
	ILocalizationServiceProvider& Provider = ILocalizationServiceModule::Get().GetProvider();
	TSharedRef<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> DownloadTargetFileOp = ILocalizationServiceOperation::Create<FDownloadLocalizationTargetFile>();
	DownloadTargetFileOp->SetInTargetGuid(AssociatedLocalizationTarget->Settings.Guid);
	DownloadTargetFileOp->SetInLocale(FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath)));
	FString Path = FPaths::ProjectSavedDir() / "Temp" / "LastImportFromLocService.po";
	FPaths::MakePathRelativeTo(Path, *FPaths::ProjectDir());
	DownloadTargetFileOp->SetInRelativeOutputFilePathAndName(Path);

	GWarn->BeginSlowTask(LOCTEXT("ImportingFromLocalizationService", "Importing Latest from Localization Service..."), true);

	Provider.Execute(DownloadTargetFileOp, TArray<FLocalizationServiceTranslationIdentifier>(), ELocalizationServiceOperationConcurrency::Asynchronous, FLocalizationServiceOperationComplete::CreateSP(this, &FTranslationEditor::DownloadLatestFromLocalizationServiceComplete));
}

void FTranslationEditor::DownloadLatestFromLocalizationServiceComplete(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result)
{
	check(AssociatedLocalizationTarget.IsValid());
	TSharedPtr<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> DownloadLocalizationTargetOp = StaticCastSharedRef<FDownloadLocalizationTargetFile>(Operation);
	bool bError = !(Result == ELocalizationServiceOperationCommandResult::Succeeded);
	FText ErrorText = FText::GetEmpty();
	if (DownloadLocalizationTargetOp.IsValid())
	{
		ErrorText = DownloadLocalizationTargetOp->GetOutErrorText();
	}
	if (!bError && ErrorText.IsEmpty())
	{
		FGuid InTargetGuid;
		FString InLocale;
		FString InRelativeOutputFilePathAndName;

		if (DownloadLocalizationTargetOp.IsValid())
		{
			InTargetGuid = DownloadLocalizationTargetOp->GetInTargetGuid();
			InLocale = DownloadLocalizationTargetOp->GetInLocale();
			InRelativeOutputFilePathAndName = DownloadLocalizationTargetOp->GetInRelativeOutputFilePathAndName();

		}
		else
		{
			bError = true;
		}

		if (InTargetGuid == AssociatedLocalizationTarget->Settings.Guid && InLocale == FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath)) && !InRelativeOutputFilePathAndName.IsEmpty())
		{
			FString AbsoluteFilePathAndName = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / InRelativeOutputFilePathAndName);
			if (FPaths::FileExists(AbsoluteFilePathAndName))
			{
				GWarn->StatusUpdate(50, 100, LOCTEXT("DownloadFromLocalizationServiceFinishedNowImporting", "Download from Localization Service Finished, Importing..."));
				ImportFromPoFile(AbsoluteFilePathAndName);
			}
			else
			{
				bError = true;
			}
		}
		else
		{
			bError = true;
		}

		if (bError)
		{
			if (ErrorText.IsEmpty())
			{
				ErrorText = LOCTEXT("DownloadLatestFromLocalizationServiceFileProcessError", "An error occured when processing the file downloaded from the Localization Service.");
			}
		}
	}
	else
	{
		bError = true;
		if (ErrorText.IsEmpty())
		{
			ErrorText = LOCTEXT("DownloadLatestFromLocalizationServiceDownloadError", "An error occured while downloading the file from the Localization Service.");
		}
	}

	GWarn->StatusUpdate(100, 100, LOCTEXT("ImportFromLocalizationServiceFinished", "Import from Localization Service Complete!"));
	GWarn->EndSlowTask();

	if (bError || !ErrorText.IsEmpty())
	{
		if (ErrorText.IsEmpty())
		{
			ErrorText = LOCTEXT("DownloadLatestFromLocalizationServiceUnspecifiedError", "An unspecified error occured when trying download and import from the Localization Service.");
		}

		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Error(ErrorText);
		TranslationEditorMessageLog.Notify(ErrorText);
	}
}

void FTranslationEditor::ExportToPortableObjectFormat_Execute()
{
	const FString PortableObjectFileDescription = LOCTEXT("PortableObjectFileDescription", "Portable Object File").ToString();
	const FString PortableObjectFileExtension = TEXT("*.po");
	const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *PortableObjectFileDescription, *PortableObjectFileExtension, *PortableObjectFileExtension);
	const FString CultureToEdit = FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath));
	FString DefaultPath = FPaths::GetPath(LocalizationConfigurationScript::GetDefaultPOPath(AssociatedLocalizationTarget.Get(), CultureToEdit));
	if (!LastExportFilePath.IsEmpty())
	{
		DefaultPath = LastExportFilePath;
	}
	TArray<FString> SaveFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bSelected = false;
	const TSharedPtr<SWindow>& ParentWindow = FSlateApplication::Get().FindWidgetWindow(PreviewTextBlock);

	// Prompt the user for the filename
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;

		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		bSelected = DesktopPlatform->SaveFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("ChooseExportLocationWindowTitle", "Choose Export Location").ToString(),
			DefaultPath,
			LocalizationConfigurationScript::GetDefaultPOFileName(AssociatedLocalizationTarget.Get()),
			FileTypes,
			EFileDialogFlags::None,
			SaveFilenames
			);
	}

	if (bSelected)
	{
		LastExportFilePath = FPaths::GetPath(SaveFilenames[0]);

		// Write translation data first to ensure all changes are exported
		if (DataManager->WriteTranslationData() && ParentWindow.IsValid() && SaveFilenames.Num() > 0)
		{
			LocalizationCommandletTasks::ExportTextForCulture(ParentWindow.ToSharedRef(), AssociatedLocalizationTarget.Get(), CultureToEdit, TOptional<FString>(SaveFilenames.Top()));
		}
		else
		{
			FNotificationInfo Info( LOCTEXT("ExportFailedError", "Translation export failed!") );
			Info.ExpireDuration = 4.f;

			const TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
			if (NotificationItem.IsValid())
			{
				NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
	}
}

void FTranslationEditor::ImportFromPortableObjectFormat_Execute()
{
	const FString PortableObjectFileDescription = LOCTEXT("PortableObjectFileDescription", "Portable Object File").ToString();
	const FString PortableObjectFileExtension = TEXT("*.po");
	const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *PortableObjectFileDescription, *PortableObjectFileExtension, *PortableObjectFileExtension);
	const FString CultureToEdit = FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath));
	FString DefaultPath = FPaths::GetPath(LocalizationConfigurationScript::GetDefaultPOPath(AssociatedLocalizationTarget.Get(), CultureToEdit));
	if (!LastImportFilePath.IsEmpty())
	{
		DefaultPath = LastImportFilePath;
	}
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	bool bOpened = false;
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;

		const TSharedPtr<SWindow>& ParentWindow = FSlateApplication::Get().FindWidgetWindow(PreviewTextBlock);
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("ChooseImportLocationWindowTitle", "Choose File to Import").ToString(),
			DefaultPath,
			LocalizationConfigurationScript::GetDefaultPOFileName(AssociatedLocalizationTarget.Get()),
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames
			);
	}

	if (bOpened && OpenFilenames.Num() > 0)
	{
		FString FileToImport = OpenFilenames[0];

		ImportFromPoFile(FileToImport);
	}
}

void FTranslationEditor::ImportFromPoFile(FString FileToImport)
{
	LastImportFilePath = FPaths::GetPath(FileToImport);

	// Write translation data first to ensure all changes are exported
	const FString CultureToEdit = FPaths::GetBaseFilename(FPaths::GetPath(ArchiveFilePath));
	const TSharedPtr<SWindow>& ParentWindow = FSlateApplication::Get().FindWidgetWindow(PreviewTextBlock);

	if (DataManager->WriteTranslationData(true) && ParentWindow.IsValid())
	{
		if (LocalizationCommandletTasks::ImportTextForCulture(ParentWindow.ToSharedRef(), AssociatedLocalizationTarget.Get(), CultureToEdit, TOptional<FString>(FileToImport)))
		{
			DataManager->LoadFromArchive(DataManager->GetAllTranslationsArray(), true, true);

			TabManager->TryInvokeTab(ChangedOnImportTabId);
			ChangedOnImportPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetChangedOnImportArray());
			// Need to re-add the columns we want to display
			ChangedOnImportPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source"));
			ChangedOnImportPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "TranslationBeforeImport"));
			ChangedOnImportPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation"));
		}
	}
	else
	{
		FNotificationInfo Info( LOCTEXT("ImportFailedError", "Translation import failed!") );
		Info.ExpireDuration = 4.f;

		const TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		if (NotificationItem.IsValid())
		{
			NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

void FTranslationEditor::OnFilterTextChanged(const FText& InFilterText)
{

}

void FTranslationEditor::OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type CommitInfo)
{
	const FString InFilterString = InFilterText.ToString();

	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (InFilterString != CurrentSearchFilter)
		{
			CurrentSearchFilter = InFilterString;

			DataManager->PopulateSearchResultsUsingFilter(InFilterString);

			if (SearchPropertyTable.IsValid())
			{
				SearchPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetSearchResultsArray());

				// Need to re-add the columns we want to display
				SearchPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source"));
				SearchPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation"));

				TArray<TSharedRef<IPropertyTableColumn>> Columns = SearchPropertyTable->GetColumns();
				for (TSharedRef<IPropertyTableColumn> Column : Columns)
				{
					Column->SetFrozen(true);
				}
			}
		}
	}
}

void FTranslationEditor::OpenSearchTab_Execute()
{
	TabManager->TryInvokeTab(SearchTabId);
}

FReply FTranslationEditor::OnGetHistoryButtonClicked()
{
	// Load the actual history data
	DataManager->GetHistoryForTranslationUnits();

	// Items might have moved from Untranslated to review, so refresh the view of both tables
	if (UntranslatedPropertyTable.IsValid())
	{
		UntranslatedPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetUntranslatedArray());

		// Need to re-add the columns we want to display
		UntranslatedPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source"));
		UntranslatedPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation"));

		TArray<TSharedRef<IPropertyTableColumn>> Columns = UntranslatedPropertyTable->GetColumns();
		for (TSharedRef<IPropertyTableColumn> Column : Columns)
		{
			Column->SetFrozen(true);
		}
	}

	if (ReviewPropertyTable.IsValid())
	{
		ReviewPropertyTable->SetObjects((TArray<UObject*>&)DataManager->GetReviewArray());

		// Need to re-add the columns we want to display
		ReviewPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Source"));
		ReviewPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "Translation"));
		ReviewPropertyTable->AddColumn((TWeakFieldPtr<FProperty>)FindFProperty<FProperty>(UTranslationUnit::StaticClass(), "HasBeenReviewed"));

		TArray<TSharedRef<IPropertyTableColumn>> Columns = ReviewPropertyTable->GetColumns();
		for (TSharedRef<IPropertyTableColumn> Column : Columns)
		{
			FString ColumnId = Column->GetId().ToString();
			if (ColumnId == "HasBeenReviewed")
			{
				Column->SetWidth(120);
				Column->SetSizeMode(EPropertyTableColumnSizeMode::Fixed);
			}
			// Freeze columns, don't want user to remove them
			Column->SetFrozen(true);
		}
	}

	// Make sure all UI is refreshed
	RefreshUI();

	// Make sure current selection is reflected
	UpdateUntranslatedSelection();
	UpdateNeedsReviewSelection();
	UpdateCompletedSelection();
	UpdateSearchSelection();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
