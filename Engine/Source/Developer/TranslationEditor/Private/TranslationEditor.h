// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Toolkits/IToolkitHost.h"
#include "TranslationDataManager.h"
#include "Styling/AppStyle.h"
#include "ITranslationEditor.h"
#include "IPropertyTable.h"
#include "CustomFontColumn.h"
#include "TranslationUnit.h"
#include "LocalizationConfigurationScript.h"

class IPropertyTableRow;
class IPropertyTableWidgetHandle;

class FTranslationEditor :  public ITranslationEditor
{
public:
	/**
	 *	Creates a new FTranslationEditor and calls Initialize
	 */
	static TSharedRef< FTranslationEditor > Create(TSharedRef< FTranslationDataManager > DataManager, const FString& InManifestFile, const FString& InArchiveFile)
	{
		TSharedRef< FTranslationEditor > TranslationEditor = MakeShareable(new FTranslationEditor(DataManager, InManifestFile, InArchiveFile, nullptr));

		// Some stuff that needs to use the "this" pointer is done in Initialize (because it can't be done in the constructor)
		TranslationEditor->Initialize();

		for (UTranslationUnit* TranslationUnit : DataManager->GetAllTranslationsArray())
		{
			// Set up a property changed event to trigger a write of the translation data when TranslationUnit property changes
			TranslationUnit->OnPropertyChanged().AddSP(DataManager, &FTranslationDataManager::HandlePropertyChanged);
		}

		return TranslationEditor;
	}
	
	static TSharedRef< FTranslationEditor > Create(TSharedRef< FTranslationDataManager > DataManager, ULocalizationTarget* const LocalizationTarget, const FString& CultureToEdit)
	{
		check(LocalizationTarget);
		
		TSharedRef< FTranslationEditor > TranslationEditor = MakeShareable(new FTranslationEditor(DataManager, LocalizationConfigurationScript::GetManifestPath(LocalizationTarget), LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, CultureToEdit), LocalizationTarget));

		// Some stuff that needs to use the "this" pointer is done in Initialize (because it can't be done in the constructor)
		TranslationEditor->Initialize();

		for (UTranslationUnit* TranslationUnit : DataManager->GetAllTranslationsArray())
		{
			// Set up a property changed event to trigger a write of the translation data when TranslationUnit property changes
			TranslationUnit->OnPropertyChanged().AddSP(DataManager, &FTranslationDataManager::HandlePropertyChanged);
		}

		return TranslationEditor;
	}

	virtual ~FTranslationEditor() = default;

	TRANSLATIONEDITOR_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	TRANSLATIONEDITOR_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The object to edit
	 */
	TRANSLATIONEDITOR_API void InitTranslationEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost );

	/** IToolkit interface */
	TRANSLATIONEDITOR_API virtual FName GetToolkitFName() const override;
	TRANSLATIONEDITOR_API virtual FText GetBaseToolkitName() const override;
	TRANSLATIONEDITOR_API virtual FText GetToolkitName() const override;
	TRANSLATIONEDITOR_API virtual FText GetToolkitToolTipText() const override;
	TRANSLATIONEDITOR_API virtual FString GetWorldCentricTabPrefix() const override;
	TRANSLATIONEDITOR_API virtual FLinearColor GetWorldCentricTabColorScale() const override;

protected:
	/** Called when "Save" is clicked for this asset */
	TRANSLATIONEDITOR_API virtual void SaveAsset_Execute() override;

private:

	FTranslationEditor(TSharedRef< FTranslationDataManager > InDataManager, const FString& InManifestFile, const FString& InArchiveFile, ULocalizationTarget* const LocalizationTarget)
		: ITranslationEditor(InManifestFile, InArchiveFile, LocalizationTarget)
		, DataManager(InDataManager)
		, SourceFont(FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle ))
		, TranslationTargetFont(FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle ))
		, SourceColumn(MakeShareable(new FCustomFontColumn(SourceFont)))
		, TranslationColumn(MakeShareable(new FCustomFontColumn(TranslationTargetFont)))
		, PreviewTextBlock(SNew(STextBlock)
				.Text(FText::FromString(""))
				.Font(TranslationTargetFont))
		, NamespaceTextBlock(SNew(STextBlock)
				.Text(FText::FromString("")))
	{}

	/** Does some things we can't do in the constructor because we can't get a SharedRef to "this" there */ 
	TRANSLATIONEDITOR_API void Initialize();

	/**	Spawns the untranslated tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_Untranslated( const FSpawnTabArgs& Args );
	/** Getter function bound to the label attribute for the untranslated tab. */
	TRANSLATIONEDITOR_API FText GetTabLabel_Untranslated() const;
	/**	Spawns the review tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_Review( const FSpawnTabArgs& Args );
	/** Getter function bound to the label attribute for the review tab. */
	TRANSLATIONEDITOR_API FText GetTabLabel_Review() const;
	/**	Spawns the completed tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_Completed( const FSpawnTabArgs& Args );
	/** Getter function bound to the label attribute for the completed tab. */
	TRANSLATIONEDITOR_API FText GetTabLabel_Completed() const;
	/**	Spawns the preview tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_Preview( const FSpawnTabArgs& Args );
	/**	Spawns the context tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_Context( const FSpawnTabArgs& Args );
	/**	Spawns the history tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_History( const FSpawnTabArgs& Args );
	/**	Spawns the search tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_Search( const FSpawnTabArgs& Args );
	/**	Spawns the Changed on Import tab */
	TRANSLATIONEDITOR_API TSharedRef<SDockTab> SpawnTab_ChangedOnImport(const FSpawnTabArgs& Args);

	/** Map actions for the UI_COMMANDS */
	TRANSLATIONEDITOR_API void MapActions();

	/** Change the font for the source language */
	TRANSLATIONEDITOR_API void ChangeSourceFont();

	/** For button delegate */
	FReply ChangeSourceFont_FReply()
	{
		ChangeSourceFont();

		return FReply::Handled();
	}

	/** Change the font for the target translation language */
	TRANSLATIONEDITOR_API void ChangeTranslationTargetFont();

	/** For button delegate */
	FReply ChangeTranslationTargetFont_FReply()
	{
		ChangeTranslationTargetFont();

		return FReply::Handled();
	}

	/** Called on SpinBox OnValueCommitted */
	void OnSourceFontSizeCommitt(int32 NewFontSize, ETextCommit::Type)
	{
		SourceFont.Size = NewFontSize;
		RefreshUI();
	}

	void OnTranslationTargetFontSizeCommitt(int32 NewFontSize, ETextCommit::Type)
	{
		TranslationTargetFont.Size = NewFontSize;
		RefreshUI();
	}

	/** Open the file dialog prompt (at the DefaultFile location) to allow the user to pick a font, then return the user's selection, and a boolean of whether something was selected */
	TRANSLATIONEDITOR_API bool OpenFontPicker(const FString DefaultFile, FString& OutFile);

	/** Reset all of the UI after a new font is chosen */
	TRANSLATIONEDITOR_API void RefreshUI();

	/** Update content when a new translation unit selection is made */
	TRANSLATIONEDITOR_API void UpdateTranslationUnitSelection(TSet<TSharedRef<IPropertyTableRow>>& SelectedRows);

	/** Update content when a new translation unit selection is made in the Untranslated PropertyTable */
	TRANSLATIONEDITOR_API void UpdateUntranslatedSelection();

	/** Update content when a new translation unit selection is made in the Needs Review PropertyTable */
	TRANSLATIONEDITOR_API void UpdateNeedsReviewSelection();

	/** Update content when a new translation unit selection is made in the Completed PropertyTable */
	TRANSLATIONEDITOR_API void UpdateCompletedSelection();

	/** Update content when a new translation unit selection is made in the Search PropertyTable */
	TRANSLATIONEDITOR_API void UpdateSearchSelection();

	/** Update content when a new translation unit selection is made in the ChangedOnImport PropertyTable */
	TRANSLATIONEDITOR_API void UpdateChangedOnImportSelection();

	/** Update content when a new context selection is made */
	TRANSLATIONEDITOR_API void UpdateContextSelection();

	/** Called when "Preview in Editor" is clicked for this Localization Target */
	TRANSLATIONEDITOR_API void PreviewAllTranslationsInEditor_Execute();
	
	/** Called when "Import from Localization Service" is clicked for this Localization Target */
	TRANSLATIONEDITOR_API void ImportLatestFromLocalizationService_Execute();

	/** Callback for when the Localization Service operation started when "Import from Localization Service" was clicked finishes */
	TRANSLATIONEDITOR_API void DownloadLatestFromLocalizationServiceComplete(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result);

	/** Called when "Export to .PO" is clicked for this Localization Target */
	TRANSLATIONEDITOR_API void ExportToPortableObjectFormat_Execute();

	/** Called when "Import from .PO" is clicked for this Localization Target */
	TRANSLATIONEDITOR_API void ImportFromPortableObjectFormat_Execute();

	/** Import from the specified .po file into this localization target */
	TRANSLATIONEDITOR_API void ImportFromPoFile(FString FileToImport);

	/** Open the search tab */
	TRANSLATIONEDITOR_API void OpenSearchTab_Execute();

	/** Called when the filter text in the SearchBox is changed */
	TRANSLATIONEDITOR_API void OnFilterTextChanged(const FText& InFilterText);

	/** Called when text is committed to the SearchBox */
	TRANSLATIONEDITOR_API void OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type CommitInfo);

	/** Called when the get history button is clicked to retrieve history from source control */
	TRANSLATIONEDITOR_API FReply OnGetHistoryButtonClicked();

	/**	The tab id for the untranslated tab */
	static TRANSLATIONEDITOR_API const FName UntranslatedTabId;
	/**	The tab id for the review tab */
	static TRANSLATIONEDITOR_API const FName ReviewTabId;
	/**	The tab id for the completed tab */
	static TRANSLATIONEDITOR_API const FName CompletedTabId;
	/**	The tab id for the preview tab */
	static TRANSLATIONEDITOR_API const FName PreviewTabId;
	/**	The tab id for the context tab */
	static TRANSLATIONEDITOR_API const FName ContextTabId;
	/**	The tab id for the history tab */
	static TRANSLATIONEDITOR_API const FName HistoryTabId;
	/**	The tab id for the search tab */
	static TRANSLATIONEDITOR_API const FName SearchTabId;
	/**	The tab id for the changed on import tab */
	static TRANSLATIONEDITOR_API const FName ChangedOnImportTabId;

	/** The Untranslated Tab */
	TWeakPtr<SDockTab> UntranslatedTab;
	/** The Review Tab */
	TWeakPtr<SDockTab> ReviewTab;
	/** The Review Tab */
	TWeakPtr<SDockTab> CompletedTab;
	/** The Search Tab */
	TWeakPtr<SDockTab> SearchTab;
	/** The Changed On Import Tab */
	TWeakPtr<SDockTab> ChangedOnImportTab;

	/** Search box for searching the source and translation strings */
	TSharedPtr<SSearchBox> SearchBox;
	/** Current search filter */
	FString CurrentSearchFilter;

	/** Manages the reading and writing of data to file */
	TSharedRef< FTranslationDataManager > DataManager;

	/** The table of untranslated items */
	TSharedPtr< class IPropertyTable > UntranslatedPropertyTable;
	/** The table of translations to review */
	TSharedPtr< class IPropertyTable > ReviewPropertyTable;
	/** The table of completed translations */
	TSharedPtr< class IPropertyTable > CompletedPropertyTable;
	/** The table of context information */
	TSharedPtr< class IPropertyTable > ContextPropertyTable;
	/** The table of previous revision information */
	TSharedPtr< class IPropertyTable > HistoryPropertyTable;
	/** The table of search results */
	TSharedPtr< class IPropertyTable > SearchPropertyTable;
	/** The table of changed on import results */
	TSharedPtr< class IPropertyTable > ChangedOnImportPropertyTable;

	/** The slate widget table of untranslated items */
	TSharedPtr< class IPropertyTableWidgetHandle > UntranslatedPropertyTableWidgetHandle;
	/** The slate widget table of translations to review */
	TSharedPtr< class IPropertyTableWidgetHandle > ReviewPropertyTableWidgetHandle;
	/** The slate widget table of completed items */
	TSharedPtr< class IPropertyTableWidgetHandle > CompletedPropertyTableWidgetHandle;
	/** The slate widget table of contexts for this item */
	TSharedPtr< class IPropertyTableWidgetHandle > ContextPropertyTableWidgetHandle;
	/** The slate widget table of previous revision information */
	TSharedPtr< class IPropertyTableWidgetHandle > HistoryPropertyTableWidgetHandle;
	/** The slate widget table of search results */
	TSharedPtr< class IPropertyTableWidgetHandle > SearchPropertyTableWidgetHandle;
	/** The slate widget table of translations that changed on import */
	TSharedPtr< class IPropertyTableWidgetHandle > ChangedOnImportPropertyTableWidgetHandle;

	/** Font to use for the source language */
	FSlateFontInfo SourceFont;
	/** Font to use for the translation target language */
	FSlateFontInfo TranslationTargetFont;

	/** Custom FontColumn for columns that display source text */
	TSharedRef<FCustomFontColumn> SourceColumn;
	/** Custom FontColumn for columns that display source text */
	TSharedRef<FCustomFontColumn> TranslationColumn;

	/** Text block for previewing the currently selected translation */
	TSharedRef<STextBlock> PreviewTextBlock;
	/** Text block displaying the namespace of the currently selected translation unit */
	TSharedRef<STextBlock> NamespaceTextBlock;

	/** Used to remember the location of the file the user last exported to */
	FString LastExportFilePath;
	/** Used to remember the location of the file the user last imported */
	FString LastImportFilePath;
};

#endif // WITH_EDITOR
