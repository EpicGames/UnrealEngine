// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR

#include "TranslationEditorMenu.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Settings/EditorExperimentalSettings.h"
#include "TranslationPickerWidget.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "TranslationEditor.h"

#define LOCTEXT_NAMESPACE "TranslationEditorToolbar"

void FTranslationEditorMenu::FillTranslationMenu( FMenuBuilder& MenuBuilder/*, FTranslationEditor& TranslationEditor*/ )
{
	MenuBuilder.BeginSection("Font", LOCTEXT("Translation_FontHeading", "Font"));
	{
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().ChangeSourceFont );
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().ChangeTranslationTargetFont );
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().PreviewAllTranslationsInEditor );
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().ImportLatestFromLocalizationService );
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().ExportToPortableObjectFormat );
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().ImportFromPortableObjectFormat );
		MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().OpenSearchTab );
		if (GetDefault<UEditorExperimentalSettings>()->bEnableTranslationPicker)
		{
			MenuBuilder.AddMenuEntry(FTranslationEditorCommands::Get().OpenTranslationPicker);
		}
	}
	MenuBuilder.EndSection();
}

void FTranslationEditorMenu::SetupTranslationEditorMenu( TSharedPtr< FExtender > Extender, FTranslationEditor& TranslationEditor)
{
	// Add additional editor menu
	{
		struct Local
		{
			static void AddSaveMenuOption( FMenuBuilder& MenuBuilder )
			{
				MenuBuilder.AddMenuEntry( FTranslationEditorCommands::Get().SaveTranslations, "SaveTranslations", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset") );
			}

			static void AddTranslationEditorMenu( FMenuBarBuilder& MenuBarBuilder )
			{
				// View
				MenuBarBuilder.AddPullDownMenu( 
					LOCTEXT("TranslationMenu", "Translation"),
					LOCTEXT("TranslationMenu_ToolTip", "Open the Translation menu"),
					FNewMenuDelegate::CreateStatic( &FTranslationEditorMenu::FillTranslationMenu ),
					"View");
			}
		};

		Extender->AddMenuExtension(
			"FileLoadAndSave",
			EExtensionHook::First,
			TranslationEditor.GetToolkitCommands(),
			FMenuExtensionDelegate::CreateStatic( &Local::AddSaveMenuOption ) );

		Extender->AddMenuBarExtension(
			"Edit",
			EExtensionHook::After,
			TranslationEditor.GetToolkitCommands(), 
			FMenuBarExtensionDelegate::CreateStatic( &Local::AddTranslationEditorMenu ) );
	}
}

void FTranslationEditorMenu::SetupTranslationEditorToolbar( TSharedPtr< FExtender > Extender, FTranslationEditor& TranslationEditor )
{
	struct Local
	{
		static void AddToolbarButtons( FToolBarBuilder& ToolbarBuilder )
		{
			ToolbarBuilder.AddToolBarButton(
				FTranslationEditorCommands::Get().SaveTranslations, "SaveTranslations", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset"));
			ToolbarBuilder.AddToolBarButton(
				FTranslationEditorCommands::Get().PreviewAllTranslationsInEditor, "PreviewTranslationsInEditor", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.PreviewInEditor")); 
			ToolbarBuilder.AddToolBarButton(
				FTranslationEditorCommands::Get().ImportLatestFromLocalizationService, "ImportLatestFromLocalizationService", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.ImportLatestFromLocalizationService"));
			ToolbarBuilder.AddToolBarButton(
				FTranslationEditorCommands::Get().ExportToPortableObjectFormat, "ExportToPortableObjectFormat", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.Export"));
			ToolbarBuilder.AddToolBarButton(
				FTranslationEditorCommands::Get().ImportFromPortableObjectFormat, "ImportFromPortableObjectFormat", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.Import"));
			ToolbarBuilder.AddToolBarButton(
				FTranslationEditorCommands::Get().OpenSearchTab, "OpenSearchTab", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.Search"));
			if (GetDefault<UEditorExperimentalSettings>()->bEnableTranslationPicker)
			{
				ToolbarBuilder.AddWidget(SNew(STranslationWidgetPicker));
			}
		}
	};

	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::First,
		TranslationEditor.GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic( &Local::AddToolbarButtons ) );
}


//////////////////////////////////////////////////////////////////////////
// FTranslationEditorCommands

void FTranslationEditorCommands::RegisterCommands() 
{
	UI_COMMAND( ChangeSourceFont, "Change Source Font", "Change the Font for the Source Lanugage", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ChangeTranslationTargetFont, "Change Translation Font", "Change the Translation Target Language Font", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SaveTranslations, "Save", "Saves the translations to file", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( PreviewAllTranslationsInEditor, "Preview in Editor", "Preview All Translations in the Editor UI", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ImportLatestFromLocalizationService, "Import from Translation Service", "Download and Import Latest Translations From Localization Service. (Localization Service settings can be modified in the Localization Dashboard)", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ExportToPortableObjectFormat, "Export to .PO", "Export to Portable Object Format", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ImportFromPortableObjectFormat, "Import from .PO", "Import from Portable Object Format", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenSearchTab, "Search", "Search Source and Translation Strings", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenTranslationPicker, "Translation Picker", "Open the Translation Picker to Modify Editor Translations", EUserInterfaceActionType::Button, FInputChord() )
}


#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
