// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslationEditorModule.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "TranslationDataManager.h"
#include "TranslationPickerWidget.h"
#include "TranslationEditor.h"
#include "MessageLogModule.h"
#include "ITranslationEditor.h"

class FTranslationEditor;

IMPLEMENT_MODULE( FTranslationEditorModule, TranslationEditor );

#define LOCTEXT_NAMESPACE "TranslationEditorModule"

#if WITH_EDITOR
const FName FTranslationEditorModule::TranslationEditorAppIdentifier( TEXT( "TranslationEditorApp" ) );
#endif // WITH_EDITOR

void FTranslationEditorModule::StartupModule()
{
#if WITH_UNREAL_DEVELOPER_TOOLS
	// create a message log for source control to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing("TranslationEditor", LOCTEXT("TranslationEditorLogLabel", "Translation Editor"));
#endif

#if WITH_EDITOR
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolbarExtensibilityManager = MakeShareable(new FExtensibilityManager);
#endif // WITH_EDITOR
}

void FTranslationEditorModule::ShutdownModule()
{
#if WITH_EDITOR
	MenuExtensibilityManager.Reset();
#endif // WITH_EDITOR
	TranslationPickerManager::RemoveOverlay();
	TranslationPickerManager::ClosePickerWindow();

#if WITH_UNREAL_DEVELOPER_TOOLS
	// unregister message log
	FMessageLogModule* MessageLogModule = FModuleManager::LoadModulePtr<FMessageLogModule>("MessageLog");
	if (MessageLogModule)
	{
		MessageLogModule->UnregisterLogListing("TranslationEditor");
	}
#endif
}

#if WITH_EDITOR
TSharedRef<FTranslationEditor> FTranslationEditorModule::CreateTranslationEditor(const FString& ManifestFile, const FString& NativeArchiveFile, const FString& ArchiveFileToEdit, bool& OutLoadedSuccessfully)
{
	TSharedRef< FTranslationDataManager > DataManager = MakeShareable( new FTranslationDataManager(ManifestFile, NativeArchiveFile, ArchiveFileToEdit) );
	OutLoadedSuccessfully = DataManager->GetLoadedSuccessfully();

	GWarn->BeginSlowTask(LOCTEXT("BuildingUserInterface", "Building Translation Editor UI..."), true);

	TSharedRef< FTranslationEditor > NewTranslationEditor(FTranslationEditor::Create(DataManager, ManifestFile, ArchiveFileToEdit));
	NewTranslationEditor->InitTranslationEditor(EToolkitMode::Standalone, TSharedPtr<IToolkitHost>());

	GWarn->EndSlowTask();

	return NewTranslationEditor;
}

TSharedRef<FTranslationEditor> FTranslationEditorModule::CreateTranslationEditor(ULocalizationTarget* const LocalizationTarget, const FString& CultureToEdit, bool& OutLoadedSuccessfully)
{
	const FString ManifestFile = LocalizationConfigurationScript::GetManifestPath(LocalizationTarget);
	FString NativeCultureName;
	if (LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
	{
		NativeCultureName = LocalizationTarget->Settings.SupportedCulturesStatistics[LocalizationTarget->Settings.NativeCultureIndex].CultureName;
	}
	const FString NativeArchiveFile = NativeCultureName.IsEmpty() ? FString() : LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, NativeCultureName);
	const FString ArchiveFileToEdit = LocalizationConfigurationScript::GetArchivePath(LocalizationTarget, CultureToEdit);

	TSharedRef< FTranslationDataManager > DataManager = MakeShareable( new FTranslationDataManager(LocalizationTarget, CultureToEdit) );
	OutLoadedSuccessfully = DataManager->GetLoadedSuccessfully();

	GWarn->BeginSlowTask(LOCTEXT("BuildingUserInterface", "Building Translation Editor UI..."), true);

	TSharedRef< FTranslationEditor > NewTranslationEditor(FTranslationEditor::Create(DataManager, LocalizationTarget, CultureToEdit));
	NewTranslationEditor->InitTranslationEditor(EToolkitMode::Standalone, TSharedPtr<IToolkitHost>());

	GWarn->EndSlowTask();

	return NewTranslationEditor;
}
#endif // WITH_EDITOR

void FTranslationEditorModule::OpenTranslationPicker()
{
	ITranslationEditor::OpenTranslationPicker();
}

#undef LOCTEXT_NAMESPACE
