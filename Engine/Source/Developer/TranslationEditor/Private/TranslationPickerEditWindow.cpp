// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslationPickerEditWindow.h"

#include "Brushes/SlateColorBrush.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ILocalizationServiceModule.h"
#include "ILocalizationServiceProvider.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "TranslationDataManager.h"
#include "TranslationPickerWidget.h"
#include "TranslationUnit.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TranslationPickerEditWindow)

struct FGeometry;

#define LOCTEXT_NAMESPACE "TranslationPicker"

TSharedPtr<FTranslationPickerSettingsManager> FTranslationPickerSettingsManager::TranslationPickerSettingsManagerInstance;

// Default dimensions of the Translation Picker edit window (floating window also uses these sizes, so it matches roughly)
const int32 STranslationPickerEditWindow::DefaultEditWindowWidth = 500;
const int32 STranslationPickerEditWindow::DefaultEditWindowHeight = 500;

class FTranslationPickerEditInputProcessor : public IInputProcessor
{
public:
	FTranslationPickerEditInputProcessor(STranslationPickerEditWindow* InOwner)
		: Owner(InOwner)
	{
	}

	void SetOwner(STranslationPickerEditWindow* InOwner)
	{
		Owner = InOwner;
	}

	virtual ~FTranslationPickerEditInputProcessor() = default;

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override
	{
	}

	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override
	{
		if (!Owner)
		{
			return false;
		}

		FKey Key = InKeyEvent.GetKey();

		if (Key == EKeys::Escape)
		{
			Owner->Exit();
			return true;
		}
		else if (Key == EKeys::Enter)
		{
			Owner->RestorePicker();
			return true;
		}
		else if (Key == EKeys::BackSpace)
		{
			TranslationPickerManager::bDrawBoxes = !TranslationPickerManager::bDrawBoxes;
			return true;
		}

		return false;
	}

	virtual const TCHAR* GetDebugName() const override { return TEXT("TranslationPicker"); }

private:
	STranslationPickerEditWindow* Owner;
};

UTranslationPickerSettings::UTranslationPickerSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void STranslationPickerEditWindow::Construct(const FArguments& InArgs)
{
	ParentWindow = InArgs._ParentWindow;
	WindowContents = SNew(SBox);
	TSharedRef<SVerticalBox> TextsBox = SNew(SVerticalBox);
	UTranslationPickerSettings* TranslationPickerSettings = FTranslationPickerSettingsManager::Get()->GetSettings();

	bool bShowLocServiceCheckbox = ILocalizationServiceModule::Get().GetProvider().IsEnabled();

	if (!FParse::Param(FCommandLine::Get(), TEXT("AllowTranslationPickerSubmissionsToOneSky")))
	{
		bShowLocServiceCheckbox = false;
		TranslationPickerSettings->bSubmitTranslationPickerChangesToLocalizationService = false;
	}

	TSharedPtr<SEditableTextBox> TextBox;
	float DefaultPadding = 0.0f;

	// Layout the Translation Picker Edit Widgets and some save/close buttons below them
	WindowContents->SetContent(
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(FilterBox, SSearchBox)
				.HintText(LOCTEXT("FilterBox_Hint", "Filter text entries"))
				.ToolTipText(LOCTEXT("FilterBox_ToolTip", "Type here to filter the list of text entries."))
				.SelectAllTextWhenFocused(false)
				.OnTextChanged(this, &STranslationPickerEditWindow::FilterBox_OnTextChanged)
				.OnTextCommitted(this, &STranslationPickerEditWindow::FilterBox_OnTextCommitted)
			]

			+SVerticalBox::Slot()
			.FillHeight(1.0f)		// Stretch the list vertically to fill up the user-resizable space
			[
				SAssignNew(TextListView, STextListView)
				.ListItemsSource(&FilteredItems)
				.OnGenerateRow(this, &STranslationPickerEditWindow::TextListView_OnGenerateRow)
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(DefaultPadding)
			[
				SNew(SVerticalBox)
				
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.Padding(DefaultPadding)
				[
					SNew(SHorizontalBox)
					.Visibility(bShowLocServiceCheckbox ? EVisibility::Visible : EVisibility::Collapsed)
					
					+SHorizontalBox::Slot()
					.Padding(FMargin(3, 3, 3, 3))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Center)
						.IsChecked(TranslationPickerSettings->bSubmitTranslationPickerChangesToLocalizationService ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
						.ToolTipText(LOCTEXT("SubmitTranslationPickerChangesToLocalizationServiceToolTip", "Submit changes to localization service"))
						.OnCheckStateChanged_Lambda([&](ECheckBoxState CheckedState)
						{
							UTranslationPickerSettings* TranslationPickerSettingsLocal = FTranslationPickerSettingsManager::Get()->GetSettings();
							TranslationPickerSettingsLocal->bSubmitTranslationPickerChangesToLocalizationService = CheckedState == ECheckBoxState::Checked;
							TranslationPickerSettingsLocal->SaveConfig();
						}
						)
					]
					
					+SHorizontalBox::Slot()
					.Padding(FMargin(0, 0, 3, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SubmitTranslationPickerChangesToLocalizationService", "Save to Localization Service"))
						.ToolTipText(LOCTEXT("SubmitTranslationPickerChangesToLocalizationServiceToolTip", "Submit changes to localization service"))
					]
				]
				
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(FMargin(0, 5))
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					
					+SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &STranslationPickerEditWindow::SaveAllAndExit)
						.Text(LOCTEXT("SaveAllAndClose", "Save All and Close"))
#if WITH_EDITOR
						.Visibility(EVisibility::Visible)
#else
						.Visibility(EVisibility::Hidden)
#endif // WITH_EDITOR
					]
					
					+SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &STranslationPickerEditWindow::Exit)
						.Text(LOCTEXT("CancelButton", "Cancel"))
					]
				]
			]
		]
	);

	ChildSlot
	[
		WindowContents.ToSharedRef()
	];

	InputProcessor = MakeShared<FTranslationPickerEditInputProcessor>(this);
	FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor, 0);

	UpdateListItems();

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &STranslationPickerEditWindow::SetFocusPostConstruct));
}

STranslationPickerEditWindow::~STranslationPickerEditWindow()
{
	if (InputProcessor.IsValid())
	{
		InputProcessor->SetOwner(nullptr);
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
		}
		InputProcessor.Reset();
	}
}

FReply STranslationPickerEditWindow::Close()
{
	const TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply STranslationPickerEditWindow::Exit()
{
	TranslationPickerManager::RemoveOverlay();
	Close();

	return FReply::Handled();
}

FReply STranslationPickerEditWindow::RestorePicker()
{
	Close();
	TranslationPickerManager::OpenPickerWindow();

	return FReply::Handled();
}

FReply STranslationPickerEditWindow::SaveAllAndExit()
{
	TArray<UTranslationUnit*> TempArray;

	for (TSharedPtr<FTranslationPickerTextItem> EditItem : AllItems)
	{
		UTranslationUnit* TranslationUnit = EditItem->GetTranslationUnitWithAnyChanges();
		if (TranslationUnit != nullptr && EditItem->CanSave())
		{
			TempArray.Add(TranslationUnit);
		}
	}

	if (TempArray.Num() > 0)
	{
		UTranslationPickerSettings* TranslationPickerSettings = FTranslationPickerSettingsManager::Get()->GetSettings();
		// Save the data via translation data manager
		FTranslationDataManager::SaveSelectedTranslations(TempArray, ILocalizationServiceModule::Get().GetProvider().IsEnabled() && TranslationPickerSettings->bSubmitTranslationPickerChangesToLocalizationService);
	}

	Exit();

	return FReply::Handled();
}

void STranslationPickerEditWindow::UpdateListItems()
{
	AllItems.Reset();
	FilteredItems.Reset();

	// Add a new Translation Picker Edit Widget for each picked text
	for (const FTranslationPickerTextAndGeom& PickedText : TranslationPickerManager::PickedTexts)
	{
		TSharedPtr<FTranslationPickerTextItem> Item = FTranslationPickerTextItem::BuildTextItem(PickedText.Text, true);
		
		AllItems.Add(Item);

		const FString& FilterBy = FilterText.ToString();

		if (!PickedText.Text.IsEmptyOrWhitespace() &&
			!PickedText.Text.ToString().Contains(FilterBy) &&
			!PickedText.Text.BuildSourceString().Contains(FilterBy))
		{
			continue;
		}

		FilteredItems.Add(Item);
	}

	// Update the list view if we have one
	if (TextListView.IsValid())
	{
		TextListView->RequestListRefresh();
	}
}

TSharedPtr<FTranslationPickerTextItem> FTranslationPickerTextItem::BuildTextItem(const FText& InText, bool bAllowEditing)
{
	TSharedPtr<FTranslationPickerTextItem> Item = MakeShared<FTranslationPickerTextItem>(InText, bAllowEditing);

	// Try and get the localization information for this text
	{
		if (const FString* SourceStringPtr = FTextInspector::GetSourceString(InText))
		{
			Item->SourceString = *SourceStringPtr;
		}
		Item->TranslationString = FTextInspector::GetDisplayString(InText);
		Item->TextId = FTextInspector::GetTextId(InText);
	}

	// Try and find the LocRes the active translation came from
	// We assume the LocRes is named the same as the localization target
	FString LocResPath;
#if WITH_EDITORONLY_DATA
	if (!Item->TextId.IsEmpty() && FTextLocalizationManager::Get().GetLocResID(Item->TextId.GetNamespace(), Item->TextId.GetKey(), LocResPath))
	{
		Item->LocTargetName = FPaths::GetBaseFilename(LocResPath);

		const FString CultureFilePath = FPaths::GetPath(LocResPath);
		Item->LocResCultureName = FPaths::GetBaseFilename(CultureFilePath);
	}
#endif // WITH_EDITORONLY_DATA

	// Clean the package localization ID from the namespace (to mirror what the text gatherer does when scraping for translation data)
	Item->CleanNamespace = TextNamespaceUtil::StripPackageNamespace(Item->TextId.GetNamespace().ToString());

	// Save the necessary data in UTranslationUnit for later.  This is what we pass to TranslationDataManager to save our edits
	Item->TranslationUnit = NewObject<UTranslationUnit>();
	Item->TranslationUnit->Namespace = Item->CleanNamespace;
	Item->TranslationUnit->Key = Item->TextId.GetKey().ToString();
	Item->TranslationUnit->Source = Item->SourceString;
	Item->TranslationUnit->Translation = Item->TranslationString;
	Item->TranslationUnit->LocresPath = LocResPath;

#if WITH_EDITOR
	// Can only save if we have have an identity and are in a known localization target file
	Item->bHasRequiredLocalizationInfoForSaving = !Item->TextId.IsEmpty() && !Item->LocTargetName.IsEmpty();
#endif // WITH_EDITOR

	return Item;
}

EActiveTimerReturnType STranslationPickerEditWindow::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (FilterBox.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(FilterBox, EFocusCause::SetDirectly);
	}

	return EActiveTimerReturnType::Stop;
}

void STranslationPickerEditWindow::FilterBox_OnTextChanged(const FText& InText)
{
	FilterText = InText;

	UpdateListItems();
}

void STranslationPickerEditWindow::FilterBox_OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	FilterBox_OnTextChanged(InText);
}

void STranslationPickerEditWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FTranslationPickerTextItem> InListItem)
{
	Item = InListItem;

	STableRow<TSharedPtr<FTranslationPickerTextItem>>::Construct(STableRow<TSharedPtr<FTranslationPickerTextItem>>::FArguments(), InOwnerTable);

	SetBorderImage(FAppStyle::GetBrush("WhiteBrush"));
	SetBorderBackgroundColor(FLinearColor(FColor(36, 36, 36, 255)));	// EStyleColor::Panel mot available in game

#if WITH_EDITOR
	const FTextBlockStyle& BoldText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("RichTextBlock.Bold");
#else
	// Bold text not available in game, suppress errors on 'RichTextBlock.Bold'
	FTextBlockStyle BoldText = FTextBlockStyle::GetDefault();
	BoldText.SetColorAndOpacity(FLinearColor(FColorList::White));
	FSlateColorBrush* BorderBrush = new FSlateColorBrush(FLinearColor::White);
#endif // WITH_EDITOR

	TSharedPtr<SGridPanel> GridPanel;

	// Layout all our data
	ChildSlot
	.Padding(FMargin(5))
	[
#if !WITH_EDITOR
		// The editor treats this border as the background. ie. an extra depth. Draw it in game only
		SNew(SBorder)
		.BorderBackgroundColor(FLinearColor(1, 1, 1, 0.45f))
		.BorderImage(BorderBrush)
		.Padding(FMargin(2.0f, 2.0f))
		[
#endif // WITH_EDITOR
			SNew(SBorder)
			.Padding(FMargin(5))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(FMargin(5))
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					[
						SAssignNew(GridPanel, SGridPanel)
						.FillColumn(1,1)
				
						+SGridPanel::Slot(0,0)
						.Padding(FMargin(2.5))
						.HAlign(HAlign_Right)
						[
							SNew(STextBlock)
							.TextStyle(&BoldText)
							.Text(LOCTEXT("SourceLabel", "Source:"))
						]

						+SGridPanel::Slot(0, 1)
						.Padding(FMargin(2.5))
						.HAlign(HAlign_Right)
						[
							SNew(SBox)
							// Hide translation if we don't have necessary information to modify
							.Visibility(!Item->bHasRequiredLocalizationInfoForSaving ? EVisibility::Collapsed : EVisibility::Visible)
							[
								SNew(STextBlock)
								.TextStyle(&BoldText)
#if WITH_EDITORONLY_DATA
								.Text(FText::Format(LOCTEXT("TranslationLabelWithCulture", "Translation ({0}):"), FText::AsCultureInvariant(Item->LocResCultureName)))
#else
								.Text(LOCTEXT("TranslationLabel", "Translation:"))
#endif
							]
						]
				
						+SGridPanel::Slot(1, 0)
						.Padding(FMargin(2.5))
						[
							SNew(SMultiLineEditableTextBox)
							.IsReadOnly(true)
							.Text(FText::AsCultureInvariant(Item->SourceString))
						]

						+SGridPanel::Slot(1, 1)
						.Padding(FMargin(2.5))
						[
							SNew(SBox)
							// Hide translation if we don't have necessary information to modify
							.Visibility(!Item->bHasRequiredLocalizationInfoForSaving ? EVisibility::Collapsed : EVisibility::Visible)
							[
								SAssignNew(Item->TextBox, SMultiLineEditableTextBox)
								.IsReadOnly(!Item->bAllowEditing || !Item->bHasRequiredLocalizationInfoForSaving)
								.Text(FText::AsCultureInvariant(Item->TranslationString))
								.HintText(LOCTEXT("TranslationEditTextBox_HintText", "Enter/edit translation here."))
							]
						]
					]
				]
			]
#if !WITH_EDITOR
		]
#endif // !WITH_EDITOR
	];


	if (!Item->TextId.IsEmpty())
	{
		GridPanel->AddSlot(0, 2)
			.Padding(FMargin(2.5))
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.TextStyle(&BoldText)
				.Text(LOCTEXT("NamespaceLabel", "Namespace:"))
			];
		GridPanel->AddSlot(1, 2)
			.Padding(FMargin(2.5))
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.Text(FText::AsCultureInvariant(Item->CleanNamespace))
			];
		GridPanel->AddSlot(0, 3)
			.Padding(FMargin(2.5))
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.TextStyle(&BoldText)
				.Text(LOCTEXT("KeyLabel", "Key:"))
			];
		GridPanel->AddSlot(1, 3)
			.Padding(FMargin(2.5))
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.Text(FText::AsCultureInvariant(Item->TextId.GetKey().ToString()))
			];
		
		int32 Row = 4;
		if (Item->bHasRequiredLocalizationInfoForSaving)
		{
#if WITH_EDITOR
			GridPanel->AddSlot(0, Row)
				.Padding(FMargin(2.5))
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.TextStyle(&BoldText)
					.Text(LOCTEXT("LocresFileLabel", "Target:"))
				];
			GridPanel->AddSlot(1, Row)
				.Padding(FMargin(2.5))
				[
					SNew(SEditableTextBox)
					.IsReadOnly(true)
					.Text(FText::AsCultureInvariant(Item->LocTargetName))
				];
			++Row;
#endif // WITH_EDITOR

			GridPanel->AddSlot(0, Row)
				.Padding(FMargin(2.5))
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STranslationPickerEditWidget::CopyNamespaceAndKey)
					.Visibility(Item->bAllowEditing ? EVisibility::Visible : EVisibility::Collapsed)
					.Text(LOCTEXT("CopyNamespaceAndKey", "Copy Namespace,Key"))
				];
			GridPanel->AddSlot(1, Row)
				.Padding(FMargin(2.5))
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STranslationPickerEditWidget::SaveAndPreview)
					.IsEnabled(Item->bHasRequiredLocalizationInfoForSaving)
					.Visibility(Item->bAllowEditing ? EVisibility::Visible : EVisibility::Collapsed)
					.Text(Item->bHasRequiredLocalizationInfoForSaving ? LOCTEXT("SaveAndPreviewButtonText", "Save and Preview") : LOCTEXT("SaveAndPreviewButtonDisabledText", "Cannot Save"))
				];
		}
		else
		{
			GridPanel->AddSlot(0, Row)
				.Padding(FMargin(2.5))
				.ColumnSpan(2)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TextLocalizable_RequiresGather", "This text is localizable (requires gather)."))
				];
		}
	}
	else
	{
		FText TextNotLocalizableReason = LOCTEXT("TextNotLocalizable_Generic", "This text is not localizable.");
		if (Item->PickedText.IsCultureInvariant())
		{
			TextNotLocalizableReason = LOCTEXT("TextNotLocalizable_CultureInvariant", "This text is not localizable (culture-invariant).");
		}
		else if (Item->PickedText.IsTransient())
		{
			TextNotLocalizableReason = LOCTEXT("TextNotLocalizable_Transient", "This text is not localizable (transient).");
		}
		else if (!Item->PickedText.ShouldGatherForLocalization())
		{
			TextNotLocalizableReason = LOCTEXT("TextNotLocalizable_InvalidForGather", "This text is not localizable (invalid for gather).");
		}

		GridPanel->AddSlot(0, 2)
			.Padding(FMargin(2.5))
			.ColumnSpan(2)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(TextNotLocalizableReason)
			];
	}
}

void FTranslationPickerTextItem::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TranslationUnit);
}

FReply STranslationPickerEditWidget::SaveAndPreview()
{
	// Update translation string from entered text
	Item->TranslationUnit->Translation = Item->TextBox->GetText().ToString();

#if WITH_EDITOR
	UTranslationPickerSettings* TranslationPickerSettings = FTranslationPickerSettingsManager::Get()->GetSettings();

	// Save the data via translation data manager
	TArray<UTranslationUnit*> TempArray;
	TempArray.Add(Item->TranslationUnit);
	FTranslationDataManager::SaveSelectedTranslations(TempArray, ILocalizationServiceModule::Get().GetProvider().IsEnabled() && TranslationPickerSettings->bSubmitTranslationPickerChangesToLocalizationService);
#endif // WITH_EDITOR

#if ENABLE_LOC_TESTING
	FTextLocalizationManager::Get().AddOrUpdateDisplayStringInLiveTable(Item->TranslationUnit->Namespace, Item->TranslationUnit->Key, Item->TranslationUnit->Translation, &Item->TranslationUnit->Source);

	if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("Slate.TriggerInvalidate")))
	{
		CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
	}
#endif // ENABLE_LOC_TESTING

	return FReply::Handled();
}

FReply STranslationPickerEditWidget::CopyNamespaceAndKey()
{
	const FString CopyString = FString::Printf(TEXT("%s,%s"), *Item->TranslationUnit->Namespace, *Item->TranslationUnit->Key);
	
	FPlatformApplicationMisc::ClipboardCopy(*CopyString);

	UE_LOGF(LogConsoleResponse, Display, "Copied Namespace,Key to clipboard: %ls", *CopyString);

	return FReply::Handled();
}

UTranslationUnit* FTranslationPickerTextItem::GetTranslationUnitWithAnyChanges()
{
	if (TranslationUnit)
	{
		// Update translation string from entered text
		TranslationUnit->Translation = TextBox->GetText().ToString();

		return TranslationUnit;
	}

	return nullptr;
}

TSharedRef<ITableRow> STranslationPickerEditWindow::TextListView_OnGenerateRow(TSharedPtr<FTranslationPickerTextItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STranslationPickerEditWidget, OwnerTable, InItem);
}

#undef LOCTEXT_NAMESPACE
