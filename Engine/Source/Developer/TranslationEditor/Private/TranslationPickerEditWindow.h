// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#include "TranslationPickerEditWindow.generated.h"

struct FGeometry;
struct FKeyEvent;
class FTranslationPickerEditInputProcessor;
class ITableRow;
class SBox;
class SMultiLineEditableTextBox;
class SSearchBox;
class STableViewBase;
class SWindow;
class UTranslationUnit;

#define LOCTEXT_NAMESPACE "TranslationPicker"

UCLASS(config = TranslationPickerSettings)
class UTranslationPickerSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Bool submit translation picker changes to Localization Service */
	UPROPERTY(config)
	bool bSubmitTranslationPickerChangesToLocalizationService;
};

class FTranslationPickerSettingsManager
{
	FTranslationPickerSettingsManager()
	{
		TranslationPickerSettingsObject = NewObject<UTranslationPickerSettings>();
		TranslationPickerSettingsObject->LoadConfig();
	}

public:
	void SaveSettings()
	{
		TranslationPickerSettingsObject->SaveConfig();
	}

	void LoadSettings()
	{
		TranslationPickerSettingsObject->LoadConfig();
	}

	UTranslationPickerSettings* GetSettings()
	{
		return TranslationPickerSettingsObject;
	}

	/**
	* Gets a reference to the translation picker manager instance.
	*
	* @return A reference to the translation picker manager instance.
	*/
	static inline TSharedPtr<FTranslationPickerSettingsManager>& Get()
	{
		if (!TranslationPickerSettingsManagerInstance.IsValid())
		{
			TranslationPickerSettingsManagerInstance = MakeShareable(new FTranslationPickerSettingsManager());
		}

		return TranslationPickerSettingsManagerInstance;
	}

private:

	static TSharedPtr<FTranslationPickerSettingsManager> TranslationPickerSettingsManagerInstance;

	/** Used to load and store settings for the Translation Picker */
	UTranslationPickerSettings* TranslationPickerSettingsObject;
};

/** A text item in the item list */
struct FTranslationPickerTextItem : public FGCObject
{
	FTranslationPickerTextItem(const FText& InText, bool bAllowEditing) : PickedText(InText), bAllowEditing(bAllowEditing) {}

	/** Create new text item */
	static TSharedPtr<FTranslationPickerTextItem> BuildTextItem(const FText& InText, bool bAllowEditing);

	inline bool operator==(const FTranslationPickerTextItem& Other) const
	{
		// It is sufficient to compare a subset to know they match. We can avoid comparing other fields.
		return (CleanNamespace.Equals(Other.CleanNamespace) &&
			TextId.GetKey() == Other.TextId.GetKey() &&
			SourceString.Equals(Other.SourceString));
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTranslationPickerTextItem");
	}

	/** Return the translation unit for this text, with any modifications */
	UTranslationUnit* GetTranslationUnitWithAnyChanges();

	/** Whether or not we can save */
	bool CanSave()
	{
		return bAllowEditing && bHasRequiredLocalizationInfoForSaving;
	}

	/** The FText that we are using this widget to translate */
	FText PickedText;

	/** Whether or not to show the save button*/
	bool bAllowEditing = true;

	/** Whether or not we were able to find the necessary info for saving */
	bool bHasRequiredLocalizationInfoForSaving = true;

	FTextId TextId;
	FString SourceString;
	FString TranslationString;
	FString LocTargetName;
	FString LocResCultureName;
	FString CleanNamespace;

	/** The translation we're editing represented in a UTranslationUnit object */
	TObjectPtr<UTranslationUnit> TranslationUnit;

	/** The text box for entering/modifying a translation */
	TSharedPtr<SMultiLineEditableTextBox> TextBox;
};

class STranslationPickerEditWidget : public STableRow<TSharedPtr<FTranslationPickerTextItem>>
{
public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FTranslationPickerTextItem> InListItem);

private:
	FReply SaveAndPreview();

	FReply CopyNamespaceAndKey();

	TSharedPtr<FTranslationPickerTextItem> Item;
};

/** Translation picker edit window to allow you to translate selected FTexts in place */
class STranslationPickerEditWindow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STranslationPickerEditWindow) {}

	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)

	SLATE_END_ARGS()

	virtual ~STranslationPickerEditWindow();

	void Construct(const FArguments& InArgs);

	// Default dimensions of the Translation Picker edit window (floating window also uses these sizes, so it matches roughly)
	static const int32 DefaultEditWindowWidth;
	static const int32 DefaultEditWindowHeight;

private:
	friend class FTranslationPickerEditInputProcessor;

	FReply Close();

	FReply Exit();

	/** We need to support keyboard focus to process the 'Esc' key */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/** Return to picker floating window */
	FReply RestorePicker();

	/** Save all translations and exit */
	FReply SaveAllAndExit();

	/** Update text list items */
	void UpdateListItems();

	/** On open, set the keyboard focus to the filter box */
	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);

	/** Filters the widgets when the user changes the search text box */
	void FilterBox_OnTextChanged(const FText& InText);

	/** Filters the widgets when the user hits enter or clears the search box */
	void FilterBox_OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	TSharedRef<ITableRow> TextListView_OnGenerateRow(TSharedPtr<FTranslationPickerTextItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Input processor used to capture key and mouse events */
	TSharedPtr<FTranslationPickerEditInputProcessor> InputProcessor;

	/** Handle to the window that contains this widget */
	TWeakPtr<SWindow> ParentWindow;

	/** Contents of the window */
	TSharedPtr<SBox> WindowContents;

	/** Full unfiltered list of items */
	TArray<TSharedPtr<FTranslationPickerTextItem>> AllItems;

	/** Filtered list of items */
	TArray<TSharedPtr<FTranslationPickerTextItem>> FilteredItems;

	/** List view control */
	typedef SListView<TSharedPtr<FTranslationPickerTextItem>> STextListView;
	TSharedPtr<STextListView> TextListView;

	/** Box to filter by text */
	TSharedPtr<SSearchBox> FilterBox;
	FText FilterText;
};

#undef LOCTEXT_NAMESPACE
