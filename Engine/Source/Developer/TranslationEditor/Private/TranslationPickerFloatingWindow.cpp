// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslationPickerFloatingWindow.h"

#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Engine/GameEngine.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Text/TextLayout.h"
#include "GameFramework/PlayerController.h"
#include "HAL/Platform.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Kismet/GameplayStatics.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "MathUtil.h"
#include "Math/Vector2D.h"
#if WITH_EDITOR
#include "Editor.h"
#include "SDocumentationToolTip.h"
#endif // WITH_EDITOR
#include "Rendering/DrawElements.h"
#include "SlotBase.h"
#include "Styling/WidgetStyle.h"
#include "TranslationPickerEditWindow.h"
#include "TranslationPickerWidget.h"
#include "Types/PaintArgs.h"
#include "Widgets/IToolTip.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class ICursor;
struct FGeometry;

#define LOCTEXT_NAMESPACE "TranslationPicker"

class FTranslationPickerInputProcessor : public IInputProcessor
{
public:
	FTranslationPickerInputProcessor(STranslationPickerFloatingWindow* InOwner)
		: Owner(InOwner)
	{
	}

	void SetOwner(STranslationPickerFloatingWindow* InOwner)
	{
		Owner = InOwner;
	}

	virtual ~FTranslationPickerInputProcessor() = default;

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override
	{
	}

	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override
	{
		if (!Owner)
		{
			return false;
		}

		FKey Key = InKeyEvent.GetKey();

		if (Key == EKeys::Escape)
		{
			Owner->SetViewportMouseIgnoreLook(false);
			Owner->Exit();
			return true;
		}
		else if (Key == EKeys::Enter)
		{
			if (Owner->SwitchToEditWindow())
			{
				Owner->SetViewportMouseIgnoreLook(false);
				Owner->Close();
			}
			return true;
		}
		else if (Key == EKeys::BackSpace)
		{
			TranslationPickerManager::bDrawBoxes = !TranslationPickerManager::bDrawBoxes;
			return true;
		}
		else if (Key == EKeys::Backslash)
		{
			if (Owner->bMouseLookInputIgnored)
			{
				Owner->SetViewportMouseIgnoreLook(false);
			}
			else
			{
				Owner->SetViewportMouseIgnoreLook(true);
			}
			return true;
		}
		else if (InKeyEvent.IsControlDown())
		{
			const uint32* KeyCode = nullptr;
			const uint32* CharCode = nullptr;
			FInputKeyManager::Get().GetCodesFromKey(Key, KeyCode, CharCode);
			if (CharCode == nullptr)
			{
				return false;
			}

			const uint32* KeyCodeOne = nullptr;
			const uint32* CharCodeOne = nullptr;
			FInputKeyManager::Get().GetCodesFromKey(EKeys::One, KeyCodeOne, CharCodeOne);

			int32 EntryIndex = *CharCode - *CharCodeOne;
			if (EntryIndex < 0 || EntryIndex > 4 || EntryIndex >= TranslationPickerManager::PickedTexts.Num())
			{
				return false;	// Handle only first five entries, the max number of entries that fit in the floating picker
			}

			const FText& PickedText = TranslationPickerManager::PickedTexts[EntryIndex].Text;
			
			FTextId TextId = FTextInspector::GetTextId(PickedText);

			// Clean the package localization ID from the namespace (to mirror what the text gatherer does when scraping for translation data)
			FString EntryNamespace = TextNamespaceUtil::StripPackageNamespace(TextId.GetNamespace().ToString());
			FString EntryKey = TextId.GetKey().ToString();

			const FString CopyString = FString::Printf(TEXT("%s,%s"), *EntryNamespace, *EntryKey);
	
			FPlatformApplicationMisc::ClipboardCopy(*CopyString);

			UE_LOGF(LogConsoleResponse, Display, "Copied Namespace,Key to clipboard: %ls", *CopyString);

			return true;
		}

		return false;
	}

	virtual const TCHAR* GetDebugName() const override { return TEXT("TranslationPicker"); }

private:
	STranslationPickerFloatingWindow* Owner;
};

void STranslationPickerFloatingWindow::Construct(const FArguments& InArgs)
{
	ParentWindow = InArgs._ParentWindow;
	WindowContents = SNew(SToolTip);

	WindowContents->SetContentWidget(
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1.0f)		// Stretch the list vertically to fill up the user-resizable space
		[
			SAssignNew(TextListView, STextListView)
			.ListItemsSource(&TextListItems)
			.OnGenerateRow(this, &STranslationPickerFloatingWindow::TextListView_OnGenerateWidget)
			.ScrollbarVisibility(EVisibility::Collapsed)
		]

		+SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		.Padding(FMargin(5))
		[
			SNew(STextBlock)
			.Text(TranslationPickerManager::PickedTexts.Num() > 0 ?
				LOCTEXT("TranslationPickerEnterToEdit", "Press Enter to edit translations") :
				LOCTEXT("TranslationPickerHoverToViewEditEscToQuit", "Hover over text to view/edit translations, or press Esc to quit"))
			.Justification(ETextJustify::Center)
		]
	);

	ChildSlot
	[
		WindowContents.ToSharedRef()
	];

	InputProcessor = MakeShared<FTranslationPickerInputProcessor>(this);
	FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor, 0);
}

STranslationPickerFloatingWindow::~STranslationPickerFloatingWindow()
{
	if (InputProcessor.IsValid())
	{
		InputProcessor->SetOwner(nullptr);
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
		}
		InputProcessor.Reset();
	}
}

FReply STranslationPickerFloatingWindow::Close()
{
	const TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
	TranslationPickerManager::ResetPickerWindow();

	return FReply::Handled();
}

FReply STranslationPickerFloatingWindow::Exit()
{
	TranslationPickerManager::RemoveOverlay();
	Close();

	return FReply::Handled();
}

void STranslationPickerFloatingWindow::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	const FVector2f MousePos = FSlateApplication::Get().GetCursorPos();
	float MousePosDiffX = FMath::Abs(MousePos.X - MousePosPrev.X);
	float MousePosDiffY = FMath::Abs(MousePos.Y - MousePosPrev.Y);

	if (MousePosDiffX >= 1.0f || MousePosDiffY >= 1.0f)
	{
		MousePosPrev = MousePos;

		FWidgetPath Path = FSlateApplication::Get().LocateWindowUnderMouse(MousePos, FSlateApplication::Get().GetInteractiveTopLevelWindows(), true);

		if (Path.IsValid())
		{
			// If the path of widgets we're hovering over changed since last time (or if this is the first tick and LastTickHoveringWidgetPath hasn't been set yet)
			if (!LastTickHoveringWidgetPath.IsValid() || LastTickHoveringWidgetPath.ToWidgetPath().ToString() != Path.ToString())
			{
				// Clear all previous text and widgets
				TranslationPickerManager::PickedTexts.Reset();

				// Process the leaf-widget under the cursor
				TSharedRef<SWidget> PathWidget = Path.Widgets.Last().Widget;

				// General Widget case
				PickTextFromWidget(PathWidget, Path, false);

				// Tooltip case
				TSharedPtr<IToolTip> Tooltip = PathWidget->GetToolTip();
				if (Tooltip.IsValid() && !Tooltip->IsEmpty())
				{
					PickTextFromWidget(Tooltip->AsWidget(), Path, true);
				}

				// Also include tooltips from parent widgets in this path (since they may be visible)
				for (int32 ParentPathIndex = Path.Widgets.Num() - 2; ParentPathIndex >= 0; --ParentPathIndex)
				{
					TSharedRef<SWidget> ParentPathWidget = Path.Widgets[ParentPathIndex].Widget;

					// Tooltip case
					TSharedPtr<IToolTip> ParentTooltip = ParentPathWidget->GetToolTip();
					if (ParentTooltip.IsValid() && !ParentTooltip->IsEmpty())
					{
						PickTextFromWidget(ParentTooltip->AsWidget(), Path, true);
					}
				}
			}

			TranslationPickerManager::PickedTexts.Sort([this, MousePos](const FTranslationPickerTextAndGeom& LHS, const FTranslationPickerTextAndGeom& RHS)
			{
				FSlateRect RectLHS = GetRect(LHS.Geometry);
				FSlateRect RectRHS = GetRect(RHS.Geometry);
				if (IsNearlyEqual(RectLHS, RectRHS))
				{
					FString SourceStringLHS;
					FString SourceStringRHS;
					if (const FString* SourceStringPtrLHS = FTextInspector::GetSourceString(LHS.Text))
					{
						SourceStringLHS = *SourceStringPtrLHS;
					}
					if (const FString* SourceStringPtrRHS = FTextInspector::GetSourceString(RHS.Text))
					{
						SourceStringRHS = *SourceStringPtrRHS;
					}
					return (SourceStringLHS < SourceStringRHS);
				}

				bool bContainsLHS = false;
				bool bContainsRHS = false;
				float DistLHS = DistSquaredToRect(RectLHS, MousePos, bContainsLHS);
				float DistRHS = DistSquaredToRect(RectRHS, MousePos, bContainsRHS);

				if ((bContainsLHS && bContainsRHS) ||
					(!bContainsLHS && !bContainsRHS))
				{
					return DistLHS < DistRHS;
				}
				else if (bContainsLHS && !bContainsRHS)
				{
					return true;
				}
				else if (!bContainsLHS && bContainsRHS)
				{
					return false;
				}
				ensure(false);		// All cases handled above
				return true;
			});

			UpdateListItems();
		}

		LastTickHoveringWidgetPath = FWeakWidgetPath(Path);
	}

	if (ParentWindow.IsValid())
	{
		FVector2D WindowSize = ParentWindow.Pin()->GetSizeInScreen();
		FVector2D DesiredPosition = FSlateApplication::Get().GetCursorPos();
		DesiredPosition.X -= FSlateApplication::Get().GetCursorSize().X;
		DesiredPosition.Y += FSlateApplication::Get().GetCursorSize().Y;

		// Move to opposite side of the cursor than the tool tip, so they don't overlap
		DesiredPosition.X -= WindowSize.X;

		// Clamp to work area
		DesiredPosition = FSlateApplication::Get().CalculateTooltipWindowPosition(FSlateRect(DesiredPosition, DesiredPosition), WindowSize, false);

		// also kind of a hack, but this is the only way at the moment to get a 'cursor decorator' without using the drag-drop code path
		ParentWindow.Pin()->MoveWindowTo(DesiredPosition);
	}
}

FVector2f STranslationPickerFloatingWindow::GetNearestPoint(const FSlateRect& Rect, const FVector2f& Point, bool& Contains)
{
	FVector2f Result = Point;
	Contains = true;

	// Check each dimension individually, clamping the point to the nearest edge endpoint
	if (Point.X < Rect.Left)
	{
		Result.X = Rect.Left;
		Contains = false;
	}
	if (Point.X > Rect.Right)
	{
		Result.X = Rect.Right;
		Contains = false;
	}
	if (Point.Y < Rect.Top)
	{
		Result.Y = Rect.Top;
		Contains = false;
	}
	if (Point.Y > Rect.Bottom)
	{
		Result.Y = Rect.Bottom;
		Contains = false;
	}

	if (!Contains)
	{
		return Result;
	}

	// Point is inside the rectangle, find the position on the nearest edge
	float DistTop =		FMath::Abs(Point.Y - Rect.Top);
	float DistBottom =	FMath::Abs(Point.Y - Rect.Bottom);
	float DistLeft =	FMath::Abs(Point.X - Rect.Left);
	float DistRight =	FMath::Abs(Point.X - Rect.Right);
	if (DistTop <= DistBottom && DistTop <= DistLeft && DistTop <= DistRight)
	{
		Result.Y = Rect.Top;
	}
	else if (DistBottom <= DistLeft && DistBottom <= DistRight)
	{
		Result.Y = Rect.Bottom;
	}
	else if (DistLeft <= DistRight)
	{
		Result.X = Rect.Left;
	}
	else
	{
		Result.X = Rect.Right;
	}

	return Result;
}

float STranslationPickerFloatingWindow::DistSquaredToRect(const FSlateRect& Rect, const FVector2f& Point, bool& Contains)
{
	if ((Rect.GetSize().X < FMathf::ZeroTolerance) ||
		(Rect.GetSize().Y < FMathf::ZeroTolerance))
	{
		Contains = false;
		return FLT_MAX;
	}

	FVector2f NearestPointOnRect = GetNearestPoint(Rect, Point, Contains);

	return FVector2f::DistSquared(NearestPointOnRect, Point);
}

bool STranslationPickerFloatingWindow::IsNearlyEqual(const FSlateRect& RectLHS, const FSlateRect& RectRHS)
{
	return FMath::IsNearlyEqual(RectLHS.Left, RectRHS.Left) &&
		FMath::IsNearlyEqual(RectLHS.Top, RectRHS.Top) &&
		FMath::IsNearlyEqual(RectLHS.Right, RectRHS.Right) &&
		FMath::IsNearlyEqual(RectLHS.Bottom, RectRHS.Bottom);
}

FSlateRect STranslationPickerFloatingWindow::GetRect(const FPaintGeometry& Geometry)
{
	FSlateRenderTransform TransformPaint = Geometry.GetAccumulatedRenderTransform();
	FVector2f Pos = FVector2f(TransformPaint.GetTranslation());
	FVector2f Size = FVector2f(Geometry.GetLocalSize()) * TransformPaint.GetMatrix().GetScale().GetVector();

	return FSlateRect(Pos.X, Pos.Y, Pos.X + Size.X, Pos.Y + Size.Y);
}

void STranslationPickerFloatingWindow::PickTextFromWidget(TSharedRef<SWidget> Widget, const FWidgetPath& Path, bool IsToolTip)
{
	auto AppendPickedTextImpl = [this](const FText& InText, TSharedPtr<SWidget> InWidget, const FWidgetPath& InPath, bool IsToolTip)
	{
		const bool bAlreadyPicked = TranslationPickerManager::PickedTexts.ContainsByPredicate([&InText](const FTranslationPickerTextAndGeom& inOther)
		{
			return inOther.Text.IdenticalTo(InText);
		});

		if (!bAlreadyPicked)
		{
			TranslationPickerManager::PickedTexts.Emplace(InText, GetPaintGeometry(InWidget, InPath, IsToolTip));
		}
	};

	auto AppendPickedText = [this, AppendPickedTextImpl](const FText& InText, TSharedPtr<SWidget> InWidget, const FWidgetPath& InPath, bool IsToolTip)
	{
		if (InText.IsEmpty())
		{
			return;
		}

		// Search the text from this widget's FText::Format history to find any source text
		TArray<FHistoricTextFormatData> HistoricFormatData;
		FTextInspector::GetHistoricFormatData(InText, HistoricFormatData);

		if (HistoricFormatData.Num() > 0)
		{
			for (const FHistoricTextFormatData& HistoricFormatDataItem : HistoricFormatData)
			{
				AppendPickedTextImpl(HistoricFormatDataItem.SourceFmt.GetSourceText(), InWidget, InPath, IsToolTip);

				for (auto It = HistoricFormatDataItem.Arguments.CreateConstIterator(); It; ++It)
				{
					const FFormatArgumentValue& ArgumentValue = It.Value();
					if (ArgumentValue.GetType() == EFormatArgumentType::Text)
					{
						AppendPickedTextImpl(ArgumentValue.GetTextValue(), InWidget, InPath, IsToolTip);
					}
				}
			}
		}
		else
		{
			AppendPickedTextImpl(InText, InWidget, InPath, IsToolTip);
		}
	};

	// Have to parse the various widget types to find the FText
	if (Widget->GetTypeAsString() == "STextBlock")
	{
		STextBlock& TextBlock = (STextBlock&)Widget.Get();
		AppendPickedText(TextBlock.GetText(), Widget.ToSharedPtr(), Path, IsToolTip);
	}
	else if (Widget->GetTypeAsString() == "SRichTextBlock")
	{
		SRichTextBlock& RichTextBlock = (SRichTextBlock&)Widget.Get();
		AppendPickedText(RichTextBlock.GetText(), Widget.ToSharedPtr(), Path, IsToolTip);
	}
	else if (Widget->GetTypeAsString() == "SToolTip")
	{
		SToolTip& ToolTipWidget = (SToolTip&)Widget.Get();
		AppendPickedText(ToolTipWidget.GetTextTooltip(), Widget.ToSharedPtr(), Path, IsToolTip);
	}
#if WITH_EDITOR
	else if (Widget->GetTypeAsString() == "SDocumentationToolTip")
	{
		SDocumentationToolTip& DocumentationToolTip = (SDocumentationToolTip&)Widget.Get();
		AppendPickedText(DocumentationToolTip.GetTextTooltip(), Widget.ToSharedPtr(), Path, IsToolTip);
	}
#endif // WITH_EDITOR
	else if (Widget->GetTypeAsString() == "SEditableText")
	{
		SEditableText& EditableText = (SEditableText&)Widget.Get();
		AppendPickedText(EditableText.GetText(), Widget.ToSharedPtr(), Path, IsToolTip);
		AppendPickedText(EditableText.GetHintText(), Widget.ToSharedPtr(), Path, IsToolTip);
	}
	else if (Widget->GetTypeAsString() == "SMultiLineEditableText")
	{
		SMultiLineEditableText& MultiLineEditableText = (SMultiLineEditableText&)Widget.Get();
		AppendPickedText(MultiLineEditableText.GetText(), Widget.ToSharedPtr(), Path, IsToolTip);
		AppendPickedText(MultiLineEditableText.GetHintText(), Widget.ToSharedPtr(), Path, IsToolTip);
	}

	// Recurse into child widgets
	PickTextFromChildWidgets(Widget, Path, IsToolTip);
}

void STranslationPickerFloatingWindow::PickTextFromChildWidgets(TSharedRef<SWidget> Widget, const FWidgetPath& Path, bool IsToolTip)
{
	FChildren* Children = Widget->GetChildren();

	FWidgetPath PathChild = Path;

	for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
	{
		TSharedRef<SWidget> ChildWidget = Children->GetChildAt(ChildIndex);

		bool bExtended = PathChild.ExtendPathTo(FWidgetMatcher(ChildWidget), EVisibility::Visible);

		// Pull out any FText from this child widget
		PickTextFromWidget(ChildWidget, PathChild, IsToolTip);
	}
}

bool STranslationPickerFloatingWindow::SwitchToEditWindow()
{
	if (TranslationPickerManager::PickedTexts.Num() == 0)
	{
		return false;
	}

	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();

	// Open a different window to allow editing of the translation
	{
		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Title(LOCTEXT("TranslationPickerEditWindowTitle", "Edit Translations"))
			.CreateTitleBar(true)
			.SizingRule(ESizingRule::UserSized);

		TSharedRef<STranslationPickerEditWindow> EditWindow = SNew(STranslationPickerEditWindow)
			.ParentWindow(NewWindow);

		NewWindow->SetContent(EditWindow);

		// Make this roughly the same size as the Edit Window, so when you press Esc to edit, the window is in basically the same size
		NewWindow->Resize(FVector2f(STranslationPickerEditWindow::DefaultEditWindowWidth, STranslationPickerEditWindow::DefaultEditWindowHeight));

		if (RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow);
		}
		NewWindow->MoveWindowTo(ParentWindow.Pin()->GetPositionInScreen());
	}

	return true;
}

void STranslationPickerFloatingWindow::SetViewportMouseIgnoreLook(bool bLookIgnore)
{
	// Avoid multiple increments/decrements to AController::IgnoreLookInput, which is a uint8
	if (bMouseLookInputIgnored == bLookIgnore)
	{
		return;
	}

	if (UWorld* World = GetWorld())
	{
		if (World->HasBegunPlay())
		{
			if (APlayerController* PlayerController = UGameplayStatics::GetPlayerController(World, 0))
			{
				PlayerController->SetIgnoreLookInput(bLookIgnore);
				bMouseLookInputIgnored = bLookIgnore;
			}
		}
	}
}

UWorld* STranslationPickerFloatingWindow::GetWorld() const
{
#if WITH_EDITOR
	if (GIsEditor && IsValid(GEditor))
	{
		if (FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext())
		{
			return PIEWorldContext->World();
		}
		return GEditor->GetEditorWorldContext().World();
	}
	else
#endif // WITH_EDITOR
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		return GameEngine->GetGameWorld();
	}
	return nullptr;
}

void STranslationPickerFloatingWindow::UpdateListItems()
{
	TextListItems.Reset();

	for (const FTranslationPickerTextAndGeom& PickedText : TranslationPickerManager::PickedTexts)
	{
		TSharedPtr<FTranslationPickerTextItem> Item = FTranslationPickerTextItem::BuildTextItem(PickedText.Text, false);

		TextListItems.Add(Item);
	}

	// Update the list view if we have one
	if (TextListView.IsValid())
	{
		TextListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> STranslationPickerFloatingWindow::TextListView_OnGenerateWidget(TSharedPtr<FTranslationPickerTextItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STranslationPickerEditWidget, OwnerTable, InItem);
}

FPaintGeometry STranslationPickerFloatingWindow::GetPaintGeometry(const TSharedPtr<SWidget>& PickedWidget, const FWidgetPath& PickedPath, bool IsToolTip) const
{
	FVector2f OffsetTooltip(0.0f);
	FVector2f OffsetMenu(0.0f);

	if (IsToolTip)
	{
		const TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(0);
		OffsetTooltip = SlateUser->GetTooltipPosition();
	}
	else if (PickedPath.GetDeepestWindow()->GetType() == EWindowType::Menu)
	{
		const FArrangedWidget& ArrangedWidget = PickedPath.Widgets[0];
		FPaintGeometry GeomPaint = PickedPath.Widgets[0].Geometry.ToPaintGeometry();
		const FSlateRenderTransform& TransformPaint = GeomPaint.GetAccumulatedRenderTransform();
		OffsetMenu = FVector2f(TransformPaint.GetTranslation());
	}

	FPaintGeometry PaintGeometry;
	if (IsToolTip)
	{
		FSlateLayoutTransform TransformTooltip = FSlateLayoutTransform(OffsetTooltip);
		PaintGeometry.AppendTransform(TransformTooltip);
	}
	else
	{
		FSlateLayoutTransform TransformMenu = FSlateLayoutTransform(OffsetMenu);

		TSharedRef<SWidget> PickedWidgetRef = PickedWidget.ToSharedRef();
		bool bFound = GetGeometry(PickedWidgetRef, PaintGeometry, TransformMenu);
		if (!bFound)
		{
			PaintGeometry = PickedWidget->GetPaintSpaceGeometry().ToPaintGeometry(TransformMenu);
		}
	}

	return PaintGeometry;
}

bool STranslationPickerFloatingWindow::GetGeometry(const TSharedRef<const SWidget>& Widget, FPaintGeometry& PaintGeometry, const FSlateLayoutTransform& TransformOffset) const
{
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(Widget);
	if (!WidgetWindow.IsValid())
	{
		return false;
	}

	while (WidgetWindow->GetParentWidget().IsValid())
	{
		TSharedRef<SWidget> CurrentWidget = WidgetWindow->GetParentWidget().ToSharedRef();
		TSharedPtr<SWindow> ParentWidgetWindow = FSlateApplication::Get().FindWidgetWindow(CurrentWidget);
		if (!ParentWidgetWindow.IsValid())
		{
			break;
		}
		WidgetWindow = ParentWidgetWindow;
	}

	TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

	FWidgetPath WidgetPath;
	if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(Widget, WidgetPath))
	{
		FArrangedWidget ArrangedWidget = WidgetPath.FindArrangedWidget(Widget).Get(FArrangedWidget::GetNullWidget());
		ArrangedWidget.Geometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));
		ArrangedWidget.Geometry.AppendTransform(TransformOffset);

		const FVector2D InflateAmount = FVector2D(1, 1) / FVector2D(ArrangedWidget.Geometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());
		PaintGeometry = ArrangedWidget.Geometry.ToInflatedPaintGeometry(InflateAmount);
		return true;
	}

	return false;
}

int32 STranslationPickerOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (!TranslationPickerManager::bDrawBoxes)
	{
		return LayerId;
	}

	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	static const FName DebugBorderBrush = TEXT("Debug.Border");
	const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(DebugBorderBrush);
	const FLinearColor BoxColorYellow = FLinearColor::Yellow;														

	const TArray<FTranslationPickerTextAndGeom>& PickedTexts = TranslationPickerManager::PickedTexts;

	for (int32 i = 0; i < PickedTexts.Num(); ++i)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			PickedTexts[i].Geometry,
			Brush,
			ESlateDrawEffect::None,
			BoxColorYellow);
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE

