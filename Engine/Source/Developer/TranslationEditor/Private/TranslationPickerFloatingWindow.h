// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Layout/PaintGeometry.h"
#include "Layout/WidgetPath.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FTranslationPickerInputProcessor;
class FWidgetStyle;
class ITableRow;
class STableViewBase;
class SToolTip;
class SWidget;
class SWindow;
struct FGeometry;
struct FTranslationPickerTextItem;

#define LOCTEXT_NAMESPACE "TranslationPicker"

/** Translation picker floating window to show details of FText(s) under cursor, and allow in-place translation via TranslationPickerEditWindow */
class STranslationPickerFloatingWindow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STranslationPickerFloatingWindow) {}

	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)

	SLATE_END_ARGS()

	virtual ~STranslationPickerFloatingWindow();

	void Construct(const FArguments& InArgs);

private:
	friend class FTranslationPickerInputProcessor;

	FReply Close();

	FReply Exit();

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Pull the FText reference out of an SWidget */
	void PickTextFromWidget(TSharedRef<SWidget> Widget, const FWidgetPath& Path, bool IsToolTip);

	/** Pull the FText reference out of the child widgets of an SWidget */
	void PickTextFromChildWidgets(TSharedRef<SWidget> Widget, const FWidgetPath& Path, bool IsToolTip);

	/** Switch from floating window to edit window */
	bool SwitchToEditWindow();

	/** Find the nearest point on the rectangle, and whether it is contained in the rect */
	static FVector2f GetNearestPoint(const FSlateRect& Rect, const FVector2f& Point, bool& Contains);

	static float DistSquaredToRect(const FSlateRect& Rect, const FVector2f& Point, bool& Contains);

	static bool IsNearlyEqual(const FSlateRect& RectLHS, const FSlateRect& RectRHS);

	static FSlateRect GetRect(const FPaintGeometry& Geometry);

	/** Update text list items */
	void UpdateListItems();

	/** Toggle 3D viewport mouse turning */
	void SetViewportMouseIgnoreLook(bool bLookIgnore);

	/** Get world from editor or engine */
	UWorld* GetWorld() const;

	FPaintGeometry GetPaintGeometry(const TSharedPtr<SWidget>& PickedWidget, const FWidgetPath& PickedPath, bool IsToolTip) const;

	bool GetGeometry(const TSharedRef<const SWidget>& Widget, FPaintGeometry& PaintGeometry, const FSlateLayoutTransform& TransformOffset) const;

	TSharedRef<ITableRow> TextListView_OnGenerateWidget(TSharedPtr<FTranslationPickerTextItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Input processor used to capture key and mouse events */
	TSharedPtr<FTranslationPickerInputProcessor> InputProcessor;

	/** Handle to the window that contains this widget */
	TWeakPtr<SWindow> ParentWindow;

	/** Contents of the window */
	TSharedPtr<SToolTip> WindowContents;

	/** List items for the text list */
	TArray<TSharedPtr<FTranslationPickerTextItem>> TextListItems;

	/** List of all texts */
	typedef SListView<TSharedPtr<FTranslationPickerTextItem>> STextListView;
	TSharedPtr<STextListView> TextListView;

	FVector2f MousePosPrev;

	/** The path widgets we were hovering over last tick */
	FWeakWidgetPath LastTickHoveringWidgetPath;

	bool bMouseLookInputIgnored = false;
};

class STranslationPickerOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STranslationPickerOverlay)
	{
		_Visibility = EVisibility::HitTestInvisible;
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs) {}

private:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
};

#undef LOCTEXT_NAMESPACE
