// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslationPickerWidget.h"

#include "Engine/GameEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "TranslationPickerEditWindow.h"
#include "TranslationPickerFloatingWindow.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

class SWidget;

#define LOCTEXT_NAMESPACE "TranslationPicker"


void STranslationWidgetPicker::Construct(const FArguments& InArgs)
{
	// Mimicking a toolbar button look

	// Icon for the picker widget button
	TSharedRef< SWidget > IconWidget =
		SNew(SImage)
		.Image(FAppStyle::GetBrush("TranslationEditor.TranslationPicker"));

	// Style settings
	FName StyleSet = FAppStyle::GetAppStyleSetName();
	FName StyleName = "Toolbar";

	FText ToolTipText = LOCTEXT("TranslationPickerTooltip", "Open the Translation Picker");

	// Create the content for our button
	TSharedRef< SWidget > ButtonContent =

		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)

			// Icon image
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
			[
				IconWidget
			]

			// Label text
			+ SVerticalBox::Slot().AutoHeight()
				.HAlign(HAlign_Center)	// Center the label text horizontally
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TranslationPicker", "Translation Picker"))
					.TextStyle(FAppStyle::Get(), FName("ToolBar.Label"))	// Smaller font for tool tip labels
					.ShadowOffset(FVector2D::UnitVector)
				]
		];

	FName CheckboxStyle = ISlateStyle::Join(StyleName, ".SToolBarButtonBlock.CheckBox.Padding");

	ChildSlot
		[
			// Create a check box
			SNew(SCheckBox)

			// Use the tool bar style for this check box
			.Style(FAppStyle::Get(), "ToolBar.ToggleButton")

			// User will have set the focusable attribute for the block, honor it
			.IsFocusable(false)

			// Pass along the block's tool-tip string
			.ToolTip(SNew(SToolTip).Text(ToolTipText))
			[
				ButtonContent
			]

			// Bind the button's "on checked" event to our object's method for this
			.OnCheckStateChanged(this, &STranslationWidgetPicker::OnCheckStateChanged)


				// Bind the check box's "checked" state to our user interface action
				.IsChecked(this, &STranslationWidgetPicker::IsChecked)

				.Padding(FAppStyle::Get().GetMargin(CheckboxStyle))
		];
}

ECheckBoxState STranslationWidgetPicker::IsChecked() const
{
	return TranslationPickerManager::IsPickerWindowOpen() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STranslationWidgetPicker::OnCheckStateChanged(const ECheckBoxState NewCheckedState)
{
	if (TranslationPickerManager::IsPickerWindowOpen())
	{
		TranslationPickerManager::ClosePickerWindow();
	}
	else
	{
		TranslationPickerManager::OpenPickerWindow();
	}
}

TSharedPtr<SWindow> TranslationPickerManager::PickerWindow;
TSharedPtr<STranslationPickerFloatingWindow> TranslationPickerManager::PickerWindowWidget;
TSharedPtr<STranslationPickerOverlay> TranslationPickerManager::MainWindowOverlay;
TArray<FTranslationPickerTextAndGeom> TranslationPickerManager::PickedTexts;
bool TranslationPickerManager::bDrawBoxes = true;

bool TranslationPickerManager::OpenPickerWindow()
{
	if (!FSlateApplicationBase::IsInitialized())
	{
		return false;
	}

	if (PickerWindow.IsValid() || PickerWindowWidget.IsValid())
	{
		return false;
	}

	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();

	// Not picking previously, launch a picker window
	{
		TSharedRef<SWindow> NewWindow = SWindow::MakeCursorDecorator();
		NewWindow->SetSizingRule(ESizingRule::FixedSize);
		// The Edit window and Floating window should be roughly the same size, so it isn't too distracting switching between them
		NewWindow->Resize(FVector2f(STranslationPickerEditWindow::DefaultEditWindowWidth, STranslationPickerEditWindow::DefaultEditWindowHeight));
		NewWindow->MoveWindowTo(FSlateApplication::Get().GetCursorPos());
		PickerWindow = NewWindow;

		NewWindow->SetContent(
			SAssignNew(PickerWindowWidget, STranslationPickerFloatingWindow)
			.ParentWindow(NewWindow));

		if (RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow);
		}
	}

	// Create overlay for drawing text boxes
	if (!MainWindowOverlay.IsValid())
	{
		MainWindowOverlay = SNew(STranslationPickerOverlay);

#if WITH_EDITOR
		if (GIsEditor && IsValid(GEditor))
		{
			RootWindow->AddOverlaySlot()
			[
				MainWindowOverlay.ToSharedRef()
			];
		}
		else
#endif // WITH_EDITOR
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			if (TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin())
			{
				GameViewportWindow->AddOverlaySlot()
				[
					MainWindowOverlay.ToSharedRef()
				];
			}
		}
	}

	return true;
}

void TranslationPickerManager::ClosePickerWindow()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RequestDestroyWindow(PickerWindow.ToSharedRef());
	}

	ResetPickerWindow();
}

void TranslationPickerManager::ResetPickerWindow()
{
	PickerWindow.Reset();

	PickerWindowWidget.Reset();
}

void TranslationPickerManager::RemoveOverlay()
{
	if (!MainWindowOverlay.IsValid())
	{
		return;
	}

#if WITH_EDITOR
	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	RootWindow->RemoveOverlaySlot(MainWindowOverlay.ToSharedRef());
	MainWindowOverlay.Reset();
#else
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if (TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin())
		{
			GameViewportWindow->RemoveOverlaySlot(MainWindowOverlay.ToSharedRef());
			MainWindowOverlay.Reset();
		}
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
