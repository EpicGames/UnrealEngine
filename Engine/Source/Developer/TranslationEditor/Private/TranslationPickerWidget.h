// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "TranslationPicker"

struct FTranslationPickerTextAndGeom;
class STranslationPickerFloatingWindow;
class STranslationPickerOverlay;
class SWindow;

enum class ECheckBoxState : uint8;

class TranslationPickerManager
{
public:
	static TSharedPtr<SWindow> PickerWindow;
	static TSharedPtr<STranslationPickerFloatingWindow> PickerWindowWidget;
	static TSharedPtr<STranslationPickerOverlay> MainWindowOverlay;

	/** The FTexts that we have found under the cursor */
	static TArray<FTranslationPickerTextAndGeom> PickedTexts;

	/** Whether to draw boxes in the overlay */
	static bool bDrawBoxes;

	static bool IsPickerWindowOpen() { return PickerWindow.IsValid(); }

	static bool OpenPickerWindow();

	static void ClosePickerWindow();

	static void ResetPickerWindow();

	static void RemoveOverlay();
};

/** Widget used to launch a 'picking' session */
class STranslationWidgetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STranslationWidgetPicker) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/**
	* Called by slate to determine if this button should appear checked
	*
	* @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
	*/
	ECheckBoxState IsChecked() const;

	/**
	* Called by Slate when this tool bar check box button is toggled
	*/
	void OnCheckStateChanged(const ECheckBoxState NewCheckedState);
};

struct FTranslationPickerTextAndGeom
{
	FText Text;
	FPaintGeometry Geometry;

	FTranslationPickerTextAndGeom(const FText& InText, const FPaintGeometry& InGeometry)
		: Text(InText)
		, Geometry(InGeometry)
	{
	}
};

#undef LOCTEXT_NAMESPACE
