// Copyright Epic Games, Inc. All Rights Reserved.
#include "TranslationUnit.h"

#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TranslationUnit)

UTranslationUnit::UTranslationUnit( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{

}

#if WITH_EDITOR
void UTranslationUnit::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (Name == GET_MEMBER_NAME_CHECKED(UTranslationUnit, Translation))
	{
		// Consider modifying the translation to be an implicit review
		HasBeenReviewed = true;
	}

	TranslationUnitPropertyChangedEvent.Broadcast(Name);
}
#endif // WITH_EDITOR
