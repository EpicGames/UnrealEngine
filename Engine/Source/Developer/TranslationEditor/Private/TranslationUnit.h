// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "TranslationUnit.generated.h"

class FLocMetadataObject;
class FName;

USTRUCT()
struct FTranslationChange
{
	GENERATED_USTRUCT_BODY()

public:

	/** The changelist of this change */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	FString Version;

	/** Date of this change */
	UPROPERTY(Category=Translation, VisibleAnywhere)//, meta=(DisplayName = "Date & Time"))
	FDateTime DateAndTime = FDateTime(0);

	/** Source at time of this change */
	UPROPERTY(Category=Translation, VisibleAnywhere, meta=(MultiLine=true))
	FString Source;

	/** Translation at time of this change */
	UPROPERTY(Category=Translation, VisibleAnywhere, meta=(MultiLine=true))
	FString Translation;
};

USTRUCT()
struct FTranslationContextInfo
{
	GENERATED_USTRUCT_BODY()

public:

	/** The key specified in LOCTEXT */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	FString Key;

	/** What file and line this translation is from */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	FString Context;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(Category=Translation, VisibleAnywhere)
	FString DevNotes;
#endif

	/** List of previous versions of the source text for this context */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	TArray<FTranslationChange> Changes;
};

UCLASS(hidecategories = Object, MinimalAPI)
class UTranslationUnit : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** The localization namespace for this translation */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	FString Namespace;

	/** The localization key for this translation */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	FString Key;

	/** Original text from the source language */
	UPROPERTY(Category=Translation, VisibleAnywhere, meta=(MultiLine=true))
	FString Source;

	/** Translations */
	UPROPERTY(Category=Translation, EditAnywhere, meta=(MultiLine=true))
	FString Translation;

	/** Contexts the source was found in */
	UPROPERTY(Category=Translation, VisibleAnywhere)
	TArray<FTranslationContextInfo> Contexts;

	/** Whether the changes have been reviewed */
	UPROPERTY(Category=Translation, EditAnywhere)
	bool HasBeenReviewed;

	/** If this Translation Unit had a different translation before import, it will be stored here */
	UPROPERTY(Category=Translation, EditAnywhere, meta=(MultiLine=true))
	FString TranslationBeforeImport;

	/** Optional: Which Locres File this translation is in */
	UPROPERTY(Category = Translation, EditAnywhere)
	FString LocresPath;

	/**
	* Returns an event delegate that is executed when a property has changed.
	*
	* @return The delegate.
	*/
	DECLARE_EVENT_OneParam(UTranslationUnit, FTranslationUnitPropertyChangedEvent, FName /*PropertyName*/);
	FTranslationUnitPropertyChangedEvent& OnPropertyChanged() { return TranslationUnitPropertyChangedEvent; }

	TSharedPtr<FLocMetadataObject> KeyMetaDataObject;

#if WITH_EDITOR
protected:
	/**
	* Called when a property on this object has been modified externally
	*
	* @param PropertyThatChanged the property that was modified
	*/
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

private:

	// Holds an event delegate that is executed when a property has changed.
	FTranslationUnitPropertyChangedEvent TranslationUnitPropertyChangedEvent;
};
