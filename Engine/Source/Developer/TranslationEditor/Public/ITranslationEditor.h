// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "LocalizationTargetTypes.h"

/** Translation Editor public interface */
class ITranslationEditor : public FAssetEditorToolkit
#else
class ITranslationEditor
#endif // WITH_EDITOR
{
public:
	TRANSLATIONEDITOR_API static void OpenTranslationPicker();

#if WITH_EDITOR
	TRANSLATIONEDITOR_API static void OpenTranslationEditor(const FString& InManifestFile, const FString& InNativeArchiveFile, const FString& InArchiveFileToEdit);
	TRANSLATIONEDITOR_API static void OpenTranslationEditor(ULocalizationTarget* const LocalizationTarget, const FString& CultureToEdit);

	ITranslationEditor(const FString& InManifestFile, const FString& InArchiveFile, ULocalizationTarget* const InAssociatedLocalizationTarget)
		: ManifestFilePath(InManifestFile)
		, ArchiveFilePath(InArchiveFile)
		, AssociatedLocalizationTarget(InAssociatedLocalizationTarget)
	{}

	virtual void OnClose() override;

protected:

	/** The path to the manifest file being used for contexts. */
	FString ManifestFilePath;
	/** The path to the archive file being edited. */
	FString ArchiveFilePath;
	/** The localization target associated with the files being used/edited, if any. */
	TWeakObjectPtr<ULocalizationTarget> AssociatedLocalizationTarget;

private:

	/** To keep track of what translation editors are open editing which manifest files */
	static TMap<FString, ITranslationEditor*> OpenTranslationEditors;
	
	/** Called on open to add this to our list of open translation editors */
	void RegisterTranslationEditor();
	
	/** Called on close to remove this from our list of open translation editors */
	void UnregisterTranslationEditor();

#endif // WITH_EDITOR
};
