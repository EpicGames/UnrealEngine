// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

class FExtender;
class FMenuBuilder;
class FTranslationEditor;

/**
 * Translation Editor menu
 */
class FTranslationEditorMenu
{
public:
	static TRANSLATIONEDITOR_API void SetupTranslationEditorMenu( TSharedPtr< FExtender > Extender, FTranslationEditor& TranslationEditor);
	static TRANSLATIONEDITOR_API void SetupTranslationEditorToolbar( TSharedPtr< FExtender > Extender, FTranslationEditor& TranslationEditor);
	
protected:

	static TRANSLATIONEDITOR_API void FillTranslationMenu( FMenuBuilder& MenuBuilder/*, FTranslationEditor& TranslationEditor*/ );
};


class FTranslationEditorCommands : public TCommands<FTranslationEditorCommands>
{
public:
	/** Constructor */
	FTranslationEditorCommands() 
		: TCommands<FTranslationEditorCommands>("TranslationEditor", NSLOCTEXT("Contexts", "TranslationEditor", "Translation Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** Switch fonts */
	TSharedPtr<FUICommandInfo> ChangeSourceFont;
	TSharedPtr<FUICommandInfo> ChangeTranslationTargetFont;

	/** Save the translations to file */
	TSharedPtr<FUICommandInfo> SaveTranslations;

	/** Save the translations to file */
	TSharedPtr<FUICommandInfo> PreviewAllTranslationsInEditor;
	
	/** Download and import the latest translations from the active Translation Service */
	TSharedPtr<FUICommandInfo> ImportLatestFromLocalizationService;

	/** Export to PortableObject format (.po) */
	TSharedPtr<FUICommandInfo> ExportToPortableObjectFormat;

	/** Import from PortableObject format (.po) */
	TSharedPtr<FUICommandInfo> ImportFromPortableObjectFormat;

	/** Open the tab for searching */
	TSharedPtr<FUICommandInfo> OpenSearchTab;

	/** Open the translation picker */
	TSharedPtr<FUICommandInfo> OpenTranslationPicker;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

#endif // WITH_EDITOR
