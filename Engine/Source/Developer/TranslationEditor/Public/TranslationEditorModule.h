// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#if WITH_EDITOR
#include "Toolkits/AssetEditorToolkit.h"
#endif // WITH_EDITOR

class FTranslationEditor;
class ULocalizationTarget;

class FTranslationEditorModule
	: public IModuleInterface
#if WITH_EDITOR
	, public IHasMenuExtensibility
#endif // WITH_EDITOR
{

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**
	 * Open the translation picker.
	 * @note Alias for ITranslationEditor::OpenTranslationPicker.
	 */
	virtual void OpenTranslationPicker();

#if WITH_EDITOR
	/**
	 * Creates an instance of translation editor object.  Only virtual so that it can be called across the DLL boundary.
	 *
	 * @param ManifestFile			The path to the manifest file to be used for contexts.
	 * @param NativeArchiveFile		The path to the archive file for the native language.
	 * @param ArchiveFileToEdit		The path to the archive file to be viewed and edited.
	 * @param OutLoadedSuccessfully	Whether or not the translation editor was able to load successfully from the .manifest and .archive files
	 * 
	 * @return	The new instance of the translation editor
	 */
	virtual TSharedRef<FTranslationEditor> CreateTranslationEditor(const FString& ManifestFile, const FString& NativeArchiveFile, const FString& ArchiveFileToEdit, bool& OutLoadedSuccessfully);

	/**
	 * Creates an instance of translation editor object.  Only virtual so that it can be called across the DLL boundary.
	 *
	 * @param LocalizationTarget	The localization target whose data is to be used and edited.
	 * @param CultureToEdit			The name of a supported culture of the localization target, whose archives should be edited.
	 * @param OutLoadedSuccessfully	Whether or not the translation editor was able to load successfully from the .manifest and .archive files
	 * 
	 * @return	The new instance of the translation editor
	 */
	virtual TSharedRef<FTranslationEditor> CreateTranslationEditor(ULocalizationTarget* const LocalizationTarget, const FString& CultureToEdit, bool& OutLoadedSuccessfully);


	/** Gets the extensibility managers for outside entities to extend translation editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolbarExtensibilityManager() { return ToolbarExtensibilityManager; }

	/** Translation Editor app identifier string */
	static const FName TranslationEditorAppIdentifier;

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolbarExtensibilityManager;
#endif // WITH_EDITOR
};
