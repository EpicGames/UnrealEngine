// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TranslationEditor : ModuleRules
{
	public TranslationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("LocalizationService");

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DesktopPlatform",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"EngineSettings",
				"InputCore",
				"Json",
				"Slate",
				"SlateCore",
				"SourceControl",
				"MessageLog",
				"LocalizationService",
				"ToolMenus",
			}
		);

		if (Target.bBuildEditor)
		{
			PublicIncludePathModuleNames.Add("LevelEditor");
			PublicIncludePathModuleNames.Add("WorkspaceMenuStructure");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"PropertyEditor",
					"EditorFramework",
					"UnrealEd",
					"GraphEditor",
					"Documentation",
					"LocalizationCommandletExecution",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"WorkspaceMenuStructure",
					"DesktopPlatform",
				}
			);
		}

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Localization",
			}
		);
	}
}
